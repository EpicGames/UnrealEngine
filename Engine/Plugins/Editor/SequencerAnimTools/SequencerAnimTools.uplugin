{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Sequencer Anim Tools",
	"Description": "Animation Tools For Sequencer and Control Rig",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": true,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		},
		{
			"Name": "SequencerScripting",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "SequencerAnimTools",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}