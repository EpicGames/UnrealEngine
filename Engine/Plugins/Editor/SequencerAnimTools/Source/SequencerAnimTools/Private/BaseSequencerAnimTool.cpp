// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseSequencerAnimTool.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/TransformGizmo.h"
#include "BaseGizmos/TransformProxy.h"
#include "InteractiveGizmoManager.h"
#include "BaseGizmos/CombinedTransformGizmo.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseSequencerAnimTool)

UBaseSequencerAnimTool::UBaseSequencerAnimTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}


void FSequencerAnimToolHelpers::CreateGizmo(const FSequencerAnimToolHelpers::FGizmoData& InData, TObjectPtr <UCombinedTransformGizmo>& OutCombinedGizmo, TObjectPtr<UTransformGizmo>& OutTRSGizmo)
{
	const bool bUseNewTRS = UEditorInteractiveGizmoManager::UsesNewTRSGizmos();
	OutTRSGizmo = bUseNewTRS ? UE::EditorTransformGizmoUtil::CreateTransformGizmo(InData.ToolManager, FString(), InData.Owner) : nullptr;
	if (OutTRSGizmo)
	{
		OutTRSGizmo->TransformGizmoSource = nullptr;
		OutTRSGizmo->SetActiveTarget(InData.TransformProxy, InData.ToolManager);
		if (UGizmoElementHitMultiTarget* HitMultiTarget = Cast< UGizmoElementHitMultiTarget>(OutTRSGizmo->HitTarget))
		{
			HitMultiTarget->GizmoTransformProxy = InData.TransformProxy;
		}
	}
	else
	{
		ETransformGizmoSubElements Elements = ETransformGizmoSubElements::StandardTranslateRotate;
		OutCombinedGizmo = InData.GizmoManager->CreateCustomTransformGizmo(Elements, InData.Owner, InData.InstanceIdentifier);
		OutCombinedGizmo->SetActiveTarget(InData.TransformProxy, InData.ToolManager);
}
}
