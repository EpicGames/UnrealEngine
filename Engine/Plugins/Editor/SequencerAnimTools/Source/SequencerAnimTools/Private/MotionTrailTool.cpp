// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionTrailTool.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"
#include "LevelEditorSequencerIntegration.h"
#include "BaseGizmos/TransformProxy.h"
#include "Sequencer/SequencerTrailHierarchy.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "Framework/Commands/UICommandList.h"
#include "InteractiveGizmoManager.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"

#include "InteractiveToolManager.h"
// for raycast into World

#include "EditorInteractiveGizmoManager.h"
#include "ScopedTransaction.h"
#include "EditorModeManager.h"
#include "UnrealClient.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/TransformGizmo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionTrailTool)

#define LOCTEXT_NAMESPACE "SequencerAnimTools"

UInteractiveTool* UMotionTrailToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UMotionTrailTool* NewTool = NewObject<UMotionTrailTool>(SceneState.ToolManager);
	UEdMode* EdMode = GetTypedOuter<UEdMode>();
	FEditorModeTools* ModeManager = nullptr;
	if (EdMode)
	{
		ModeManager = EdMode->GetModeManager();
	}
	NewTool->SetWorldGizmoModeManager(SceneState.World, SceneState.GizmoManager, ModeManager);

	return NewTool;
}

bool UMotionTrailTool::ProcessCommandBindings(const FKey Key, const bool bRepeat) const
{
	if (CommandBindings.IsValid())
	{
		FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
		return CommandBindings->ProcessCommandBindings(Key, KeyState, bRepeat);
	}
	return false;
}

void FMotionTrailCommands::RegisterCommands()
{
	UI_COMMAND(TranslateSelectedKeysLeft, "Translate Selected Keys Left", "Translate selected keys one frame to the left", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Left));
	UI_COMMAND(TranslateSelectedKeysRight, "Translate Selected Keys Right", "Translate selected keys one frame to the right", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Right));
	UI_COMMAND(FrameSelection, "Frame Selection", "Frame camera around current key selection", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(DeselectAll, "Deslect All", "Deselect All Keys", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));

}

FString UMotionTrailTool::TrailKeyTransformGizmoInstanceIdentifier = TEXT("TrailKeyTransformGizmoInstanceIdentifier");
FOnCreateAdditionalTrailHierarchies UMotionTrailTool::OnCreateAdditionalTrailHierarchies;

bool UMotionTrailTool::IsHierarchyActive(const UE::SequencerAnimTools::FTrailHierarchy* Hierarchy) const
{
	return Hierarchy && EnumHasAnyFlags(Hierarchy->GetTrailCategory(), FTrailCategoryRegistry::GetVisibleCategories());
}

void UMotionTrailTool::AddTrailHierarchy(TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy> InHierarchy)
{
	TrailHierarchies.Add(MoveTemp(InHierarchy));
}

void UMotionTrailTool::Setup()
{
	UInteractiveTool::Setup();

	LeftTarget.Target = this;
	LeftTarget.bIsLeft = true;
	RightTarget.Target = this;
	RightTarget.bIsLeft = false;

	LeftClickBehavior = NewObject<USingleClickInputBehavior>(this);
	LeftClickBehavior->SetUseLeftMouseButton(); //default
	LeftClickBehavior->Initialize(&LeftTarget);
	LeftClickBehavior->Modifiers.RegisterModifier(ShiftModifierId, FInputDeviceState::IsShiftKeyDown);
	LeftClickBehavior->Modifiers.RegisterModifier(CtrlModifierId, FInputDeviceState::IsCtrlKeyDown);
	AddInputBehavior(LeftClickBehavior);

	RightClickBehavior = NewObject<USingleClickInputBehavior>(this);
	RightClickBehavior->SetUseRightMouseButton();
	RightClickBehavior->Initialize(&RightTarget);
	RightClickBehavior->Modifiers.RegisterModifier(ShiftModifierId, FInputDeviceState::IsShiftKeyDown);
	RightClickBehavior->Modifiers.RegisterModifier(CtrlModifierId, FInputDeviceState::IsCtrlKeyDown);
	AddInputBehavior(RightClickBehavior);

	TransformProxy = NewObject<UTransformProxy>(this);

	TransformProxy->OnTransformChanged.AddUObject(this, &UMotionTrailTool::GizmoTransformChanged);
	TransformProxy->OnBeginTransformEdit.AddUObject(this, &UMotionTrailTool::GizmoTransformStarted);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &UMotionTrailTool::GizmoTransformEnded);

	FSequencerAnimToolHelpers::FGizmoData GizmoData;
	GizmoData.Owner = this;
	GizmoData.TransformProxy = TransformProxy;
	GizmoData.ToolManager = GetToolManager();
	GizmoData.GizmoManager = GizmoManager;
	GizmoData.InstanceIdentifier = UMotionTrailTool::TrailKeyTransformGizmoInstanceIdentifier;
	FSequencerAnimToolHelpers::CreateGizmo(GizmoData, TransformGizmo, TRSGizmo);

	if (TRSGizmo)
	{
		TRSGizmo->SetVisibility(false);
	}
	else if (TransformGizmo)
	{
		TransformGizmo->SetVisibility(false);
	}
	SetupIntegration();
}

void UMotionTrailTool::SetupIntegration()
{

	OnSequencersChangedHandle = FLevelEditorSequencerIntegration::Get().GetOnSequencersChanged().AddLambda([this] {
		for (const TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy>& TrailHierarchy : TrailHierarchies)
		{
			TrailHierarchy->Destroy();
		}

		TrailHierarchies.Reset();
		SequencerHierarchies.Reset();
		// TODO: kind of cheap for now, later should check with member TMap<ISequencer*, FTrailHierarchy*> TrackedSequencers
		for (TWeakPtr<ISequencer> WeakSequencer : FLevelEditorSequencerIntegration::Get().GetSequencers())
		{
			TrailHierarchies.Add_GetRef(MakeShared<UE::SequencerAnimTools::FSequencerTrailHierarchy>(WeakSequencer))->Initialize();
			SequencerHierarchies.Add(WeakSequencer.Pin().Get(), TrailHierarchies.Last().Get());

			OnCreateAdditionalTrailHierarchies.Broadcast(TrailHierarchies, WeakSequencer);
		}
	});

	for (TWeakPtr<ISequencer> WeakSequencer : FLevelEditorSequencerIntegration::Get().GetSequencers())
	{
		TrailHierarchies.Add_GetRef(MakeShared<UE::SequencerAnimTools::FSequencerTrailHierarchy>(WeakSequencer))->Initialize();
		SequencerHierarchies.Add(WeakSequencer.Pin().Get(), TrailHierarchies.Last().Get());

		OnCreateAdditionalTrailHierarchies.Broadcast(TrailHierarchies, WeakSequencer);
	}

	CommandBindings = MakeShareable(new FUICommandList);

	const FMotionTrailCommands& Commands = FMotionTrailCommands::Get();

	CommandBindings->MapAction(
		Commands.TranslateSelectedKeysLeft,
		FExecuteAction::CreateUObject(this, &UMotionTrailTool::TranslateSelectedKeysLeft),
		FCanExecuteAction::CreateUObject(this, &UMotionTrailTool::SomeKeysAreSelected));

	CommandBindings->MapAction(
		Commands.TranslateSelectedKeysRight,
		FExecuteAction::CreateUObject(this, &UMotionTrailTool::TranslateSelectedKeysRight),
		FCanExecuteAction::CreateUObject(this, &UMotionTrailTool::SomeKeysAreSelected));

	CommandBindings->MapAction(
		Commands.FrameSelection,
		FExecuteAction::CreateUObject(this, &UMotionTrailTool::FrameSelection),
		FCanExecuteAction::CreateUObject(this, &UMotionTrailTool::SomeKeysAreSelected));

	CommandBindings->MapAction(
		Commands.DeselectAll,
		FExecuteAction::CreateUObject(this, &UMotionTrailTool::SelectNone),
		FCanExecuteAction::CreateUObject(this, &UMotionTrailTool::SomeKeysAreSelected));

	CommandBindings->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateUObject(this, &UMotionTrailTool::DeleteSelectedKeys),
		FCanExecuteAction::CreateUObject(this, &UMotionTrailTool::SomeKeysAreSelected));
}


void UMotionTrailTool::Shutdown(EToolShutdownType ShutdownType)
{
	Super::Shutdown(ShutdownType);
	GizmoManager->DestroyAllGizmosByOwner(this);
	ShutdownIntegration();
}

void UMotionTrailTool::ShutdownIntegration()
{
	for (TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy>& TrailHierarchy : TrailHierarchies)
	{
		// TODO: just use dtor?
		TrailHierarchy->Destroy();
	}

	TrailHierarchies.Reset();
	FLevelEditorSequencerIntegration::Get().GetOnSequencersChanged().Remove(OnSequencersChangedHandle);
}



// Detects Ctrl and Shift key states
void UMotionTrailTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == ShiftModifierId)
	{
		bShiftModifierOn = bIsOn;
	}
	else if (ModifierID == CtrlModifierId)
	{
		bCtrlModifierOn = bIsOn;
	}
	if (bShiftModifierOn && bCtrlModifierOn)
	{
		bAltModifierOn = true;
		bShiftModifierOn = false;
		bCtrlModifierOn = false;
	}
	//ALT MODIFIER doesn't work, we can not even use FSlateApplication::Get().GetModifierKeys().IsAltDown() 
	//since if there is an alt it just bails !!!!l
	
}

FInputRayHit UMotionTrailTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit ReturnHit = FInputRayHit();
	if (GetToolManager() && GetToolManager()->GetContextQueriesAPI())
	{
		if (FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport())
		{
			HHitProxy* HitResult = FocusedViewport->GetHitProxy(ClickPos.ScreenPosition.X, ClickPos.ScreenPosition.Y);
			ReturnHit.bHit = ForEachActiveHierarchyUntil([&](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
			{
				return Hierarchy.IsHitByClick(HitResult);
			});
		}
	}

	return ReturnHit;
}

void UMotionTrailTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	FEditorViewportClient* ViewportClient = ModeManager->GetFocusedViewportClient();
	FViewport* FocusedViewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	HHitProxy* HitResult = FocusedViewport->GetHitProxy(ClickPos.ScreenPosition.X, ClickPos.ScreenPosition.Y);

	UE::SequencerAnimTools::FInputClick InputClick(bAltModifierOn, bCtrlModifierOn, bShiftModifierOn);
	if (RightTarget.bClicked)
	{
		InputClick.bIsRightMouse = true;
	}

	ForEachActiveHierarchy([&](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
	{
		Hierarchy.HandleClick(ViewportClient, HitResult, InputClick);
	});
	UpdateGizmoLocation();
}


void UMotionTrailTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	ForEachActiveHierarchy([&](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
	{
		Hierarchy.GetRenderer()->Render(RenderAPI->GetSceneView(), RenderAPI->GetPrimitiveDrawInterface());
	});
}

void UMotionTrailTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	ForEachActiveHierarchy([&](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
	{
		Hierarchy.GetRenderer()->DrawHUD(RenderAPI->GetSceneView(), Canvas);
	});
}

void UMotionTrailTool::OnTick(float DeltaTime)
{
	ForEachActiveHierarchy([](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
	{
		Hierarchy.Update();
	});
	UpdateGizmoLocation();
}

TArray<UObject*> UMotionTrailTool::GetToolProperties(bool bEnabledOnly) const
{
	return  TArray<UObject*>();
}


void UMotionTrailTool::GizmoTransformStarted(UTransformProxy* Proxy)
{
	//TransactionIndex = GEditor->BeginTransaction(nullptr, LOCTEXT("MoveMotionTrail", "Move Motion Trail"), nullptr);

	StartDragTransform = Proxy->GetTransform();

	bGizmoBeingDragged = true;
	bManipulatorMadeChange = false;
	GizmoTransform = StartDragTransform;
	ForEachActiveHierarchy([](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
	{
		Hierarchy.StartTracking();
	});
}

void UMotionTrailTool::GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	if (!bGizmoBeingDragged)
	{
		return;
	}
	GizmoTransform = Transform;
	FTransform Diff = Transform.GetRelativeTransform(StartDragTransform);
	if (Diff.Equals(FTransform::Identity, UE_KINDA_SMALL_NUMBER) == false)
	{
		FVector LocationDiff = GizmoTransform.GetLocation() - StartDragTransform.GetLocation();
		ForEachActiveHierarchy([&](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
		{
			FVector WidgetLocation = Transform.GetLocation();
			const bool bShiftIsDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
			if (Hierarchy.ApplyDelta(LocationDiff, GizmoTransform.GetRotation().Rotator(), WidgetLocation, bShiftIsDown))
			{
				bManipulatorMadeChange = true;
			}
		});
		StartDragTransform = Transform;
		UpdateGizmoLocation();
	}

}

void UMotionTrailTool::GizmoTransformEnded(UTransformProxy* Proxy)
{
	ForEachActiveHierarchy([](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
	{
		Hierarchy.EndTracking();
	});
	if (bManipulatorMadeChange && TransactionIndex != INDEX_NONE)
	{
		//GEditor->EndTransaction();
	}
	else if (TransactionIndex != INDEX_NONE)
	{
	//	GEditor->CancelTransaction(TransactionIndex);
	}
	bGizmoBeingDragged = false;
	bManipulatorMadeChange = false;
	UpdateGizmoLocation();
}

void UMotionTrailTool::UpdateGizmoVisibility()
{
	auto IsAnythingSelected = [this]()
	{
		return ForEachActiveHierarchyUntil([](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
		{
			return Hierarchy.IsAnythingSelected();
		});
	};
	
	if (TransformGizmo)
	{
		TransformGizmo->SetVisibility(IsAnythingSelected());
	}
	
	if (TRSGizmo)
	{
		TRSGizmo->SetVisibility(IsAnythingSelected());
	}
}

void UMotionTrailTool::UpdateGizmoLocation()
{
	UpdateGizmoVisibility();
	
	auto GetNewLocation = [this]()
	{
		FVector NewGizmoLocation(0., 0., 0.);
		int NumSelected = 0;
		TOptional<FQuat> NewRotation;
		FQuat LocalRotation;
		ForEachActiveHierarchy([&NumSelected, &LocalRotation, &NewRotation, &NewGizmoLocation](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
		{
			FVector Location;
			if (Hierarchy.IsAnythingSelected(Location, LocalRotation))
			{
				NewGizmoLocation += Location;
				if (NewRotation.IsSet() == false)
				{
					NewRotation = LocalRotation;
				}
				++NumSelected;
			}
		});
		if (NumSelected > 0)//-V547
		{
			NewGizmoLocation /= (double)NumSelected;
			LocalRotation = NewRotation.GetValue();
		}
		return TPair<FVector, FQuat>(NewGizmoLocation, LocalRotation);
	};
	TPair<FVector, FQuat> LocRot = GetNewLocation();
	GizmoTransform.SetLocation(LocRot.Key);
	GizmoTransform.SetRotation(LocRot.Value);;

	if (TransformGizmo)
	{
		TransformGizmo->ReinitializeGizmoTransform(GizmoTransform);
	}

	if (TRSGizmo)
	{
		UInteractiveToolManager* ToolManager = GetToolManager();
		const IToolsContextQueriesAPI* ToolsContextQueries = ToolManager ? ToolManager->GetContextQueriesAPI() : nullptr;
		const bool bWorld = ToolsContextQueries ? ToolsContextQueries->GetCurrentCoordinateSystem() == EToolContextCoordinateSystem::World : false;

		// note: this is a trick to avoid cycling notifications... (cf. UCombinedTransformGizmo::ReinitializeGizmoTransform)
		TGuardValue<bool> PivotModeGuard(TransformProxy->bSetPivotMode, true);
		if (bWorld)
		{
			const FTransform TransformNoRot(FQuat::Identity, GizmoTransform.GetTranslation(), GizmoTransform.GetScale3D());
			TransformProxy->SetTransform(TransformNoRot);			
		}
		else
		{
			TransformProxy->SetTransform(GizmoTransform);
		}
	}
}


void UMotionTrailTool::TranslateSelectedKeysLeft()
{
	if (SomeKeysAreSelected())
	{
		const FScopedTransaction Transaction(LOCTEXT("TranslateSelectedKeysLeft", "Translate Selected Keys Left"));
		ForEachActiveHierarchy([](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
		{
			Hierarchy.TranslateSelectedKeys(false);
		});
	}
}

void UMotionTrailTool::TranslateSelectedKeysRight()
{
	if (SomeKeysAreSelected())
	{
		const FScopedTransaction Transaction(LOCTEXT("TranslateSelectedKeysRight", "Translate Selected Keys Right"));
		ForEachActiveHierarchy([](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
		{
			Hierarchy.TranslateSelectedKeys(true);
		});
	}
}

void UMotionTrailTool::FrameSelection()
{
	if (SomeKeysAreSelected())
	{
		FBox Bounds(EForceInit::ForceInit);
		TArray<FVector> Positions;
		ForEachActiveHierarchy([&](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
		{
			if (Hierarchy.IsAnythingSelected(Positions, true))
			{
				for (const FVector Pos : Positions)
				{
					Bounds += Pos;
					Bounds += Pos + FVector::OneVector * 5.f;
					Bounds += Pos - FVector::OneVector * 5.f;
				}
			}
		});
		if (Bounds.IsValid)
		{
			GEditor->MoveViewportCamerasToBox(Bounds, true);
		}
	}
}

void UMotionTrailTool::SelectNone()
{
	if (SomeKeysAreSelected())
	{
		ForEachActiveHierarchy([](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
		{
			Hierarchy.SelectNone();
		});
	}
}

void UMotionTrailTool::DeleteSelectedKeys()
{
	if (SomeKeysAreSelected())
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteSelectedKeys", "Delete Selected Keys"));

		ForEachActiveHierarchy([](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
		{
			Hierarchy.DeleteSelectedKeys();
		});
	}
}

bool UMotionTrailTool::SomeKeysAreSelected() const
{
	return ForEachActiveHierarchyUntil([](UE::SequencerAnimTools::FTrailHierarchy& Hierarchy)
	{
		return Hierarchy.IsAnythingSelected();
	});
}


#undef LOCTEXT_NAMESPACE

