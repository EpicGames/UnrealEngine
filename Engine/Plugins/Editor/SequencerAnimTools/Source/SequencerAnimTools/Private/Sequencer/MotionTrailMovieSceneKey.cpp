// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionTrailMovieSceneKey.h"
#include "Channels/MovieSceneChannelData.h"
#include "MovieSceneTransformTrail.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Tools/MotionTrailOptions.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"
#include "ISequencer.h"
#include "CanvasTypes.h"
#include "MovieSceneSection.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "Misc/Optional.h"

namespace UE
{
namespace SequencerAnimTools
{

struct HMotionTrailMovieSceneKeyProxy : public HBaseTrailProxy
{
	DECLARE_HIT_PROXY();

	FTransform Transform;
	FTrailKeyInfo* KeyInfo;

	HMotionTrailMovieSceneKeyProxy(const FGuid& InGuid, const FTransform& InTransform, FTrailKeyInfo* InKeyInfo) :
		HBaseTrailProxy(InGuid, HPP_UI),
		Transform(InTransform),
		KeyInfo(InKeyInfo)
	{
	}

};
IMPLEMENT_HIT_PROXY(HMotionTrailMovieSceneKeyProxy, HBaseTrailProxy);

void FMotionTraiMovieScenelKeyTool::Render(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, bool bTrailsIsEvaluating)
{
	const bool bIsVisible = (UMotionTrailToolOptions::GetTrailOptions()->bShowKeys);

	if (!bIsVisible|| PDI == nullptr)
	{
		ClearSelection();
		return;
	}
	const FTransform OffsetTransform = FTransform::Identity;//OwningTrail->GetOffsetTransform();
	const FTransform ParentSpaceTransform = OwningTrail->GetParentSpaceTransform();
	const bool bHitTesting = PDI && PDI->IsHitTesting();
	float KeySize = UMotionTrailToolOptions::GetTrailOptions()->KeySize;
	TOptional<FVector> LastPoint;
	const float TrailThickness = UMotionTrailToolOptions::GetTrailOptions()->TrailThickness;
	const FLinearColor TrailColor = UMotionTrailToolOptions::GetTrailOptions()->DefaultColor;
	//hack for PDI->DrawPoint not scaling perspective keys see FViewElementPDI::DrawPoint
	const bool bIsPerspective = (View->ViewMatrices.GetViewToClip().M[3][3] < 1.0f) ? true : false;
	if (bIsPerspective)
	{
		const float ZoomFactor = FMath::Min<float>(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);
		KeySize = KeySize / ZoomFactor;
	}
	for (const TPair<FFrameNumber, TUniquePtr<FTrailKeyInfo>>& FrameKeyPair : Keys.Keys)
	{
		if (CachedViewFrameRange.Contains(FrameKeyPair.Value->FrameNumber))
		{
			
			FLinearColor KeyColor = CachedSelection.Contains(FrameKeyPair.Value.Get()) ? UMotionTrailToolOptions::GetTrailOptions()->SelectedKeyColor :  UMotionTrailToolOptions::GetTrailOptions()->KeyColor;
			const FTransform Transform = FrameKeyPair.Value->Transform * OffsetTransform * ParentSpaceTransform;
			if (bTrailsIsEvaluating)
			{
				if (LastPoint.IsSet())
				{
					const FVector CurPoint = Transform.GetLocation();
					PDI->DrawLine(LastPoint.GetValue(), CurPoint, TrailColor, SDPG_Foreground, TrailThickness);
					LastPoint = CurPoint;
				}
				else
				{
					LastPoint = Transform.GetLocation();
				}
			}


			if (bHitTesting)
			{
				PDI->SetHitProxy(new HMotionTrailMovieSceneKeyProxy(Guid, Transform, FrameKeyPair.Value.Get()));
			}

			PDI->DrawPoint(Transform.GetLocation(), KeyColor, KeySize, SDPG_MAX);

			if (bHitTesting)
			{
				PDI->SetHitProxy(nullptr);
			}
		}
	}
}

void FMotionTraiMovieScenelKeyTool::DrawHUD(const FSceneView* View, FCanvas* Canvas)
{
	return;	
}

bool FMotionTraiMovieScenelKeyTool::IsSelected(FVector& OutVectorPosition, FQuat& OutRotation) const
{
	if (CachedSelection.Num() > 0)
	{
		UpdateSelectedKeysTransform();
		OutVectorPosition = SelectedKeysTransform.GetLocation();
		OutRotation = SelectedKeysTransform.GetRotation();
		return true;
	}
	return false;
}

void FMotionTraiMovieScenelKeyTool::GetSelectedKeyPositions(TArray<FVector>& OutPositions) const
{
	if (CachedSelection.Num() > 0)
	{
		//leaving this around in case we also want to add an extra additive offset to move the whole trail without worrying about rotation effects
		const FTransform OffsetTransform = FTransform::Identity;//OwningTrail->GetOffsetTransform();
		const FTransform ParentSpaceTransform = OwningTrail->GetParentSpaceTransform();
		for (FTrailKeyInfo* KeyInfo : CachedSelection)
		{
			const FTransform Transform = KeyInfo->Transform * OffsetTransform * ParentSpaceTransform;
			OutPositions.Add(Transform.GetLocation());
		}
	}
}

bool FMotionTraiMovieScenelKeyTool::IsSelected() const
{
	return (CachedSelection.Num() > 0);
}

bool FMotionTraiMovieScenelKeyTool::HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, FInputClick Click)
{

	if (HMotionTrailMovieSceneKeyProxy* HitProxy = HitProxyCast<HMotionTrailMovieSceneKeyProxy>(InHitProxy))
	{
		if (HitProxy->KeyInfo && HitProxy->Guid == Guid)
		{
			//	FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), IsInLevelEditor() && !GIsTransacting);

			if (Click.bShiftIsDown)
			{
				CachedSelection.Add(HitProxy->KeyInfo);
			}
			else if (Click.bCtrlIsDown)
			{
				if (CachedSelection.Contains(HitProxy->KeyInfo))
				{
					CachedSelection.Remove(HitProxy->KeyInfo);
				}
				else
				{
					CachedSelection.Add(HitProxy->KeyInfo);
				}
			}
			else
			{
				CachedSelection.Reset();
				CachedSelection.Add(HitProxy->KeyInfo);
			}

		}
		else 
		{
			if (Click.bShiftIsDown == false && Click.bCtrlIsDown == false)
			{
				CachedSelection.Reset();
			}
			return false;
		}
		UpdateSelectedKeysTransform();
		return true;
	}
	CachedSelection.Reset();
	return false;
}

void FMotionTraiMovieScenelKeyTool::UpdateSelectedKeysTransform() const
{
	if (CachedSelection.Num() > 0)
	{

		TOptional<FQuat> NewRotation;
		const FTransform OffsetTransform = FTransform::Identity;//OwningTrail->GetOffsetTransform();
		const FTransform ParentSpaceTransform = OwningTrail->GetParentSpaceTransform();
		FVector NewGizmoLocation = FVector::ZeroVector;
		for (FTrailKeyInfo* KeyInfo : CachedSelection)
		{
			const FTransform Transform = KeyInfo->Transform * OffsetTransform * ParentSpaceTransform;
			NewGizmoLocation += Transform.GetLocation();
			if (NewRotation.IsSet() == false)
			{
				NewRotation = Transform.GetRotation();
			}
		}
		NewGizmoLocation /= (double)CachedSelection.Num();
		SelectedKeysTransform.SetLocation(NewGizmoLocation);
		SelectedKeysTransform.SetRotation(NewRotation.GetValue());
	}
}

void FMotionTraiMovieScenelKeyTool::OnSectionChanged()
{
	if (ShouldRebuildKeys())
	{
		ClearSelection();
		BuildKeys();
	}

	DirtyKeyTransforms();
}

void FMotionTraiMovieScenelKeyTool::BuildKeys()
{
	TArray<FFrameNumber> KeyTimes = SelectedKeyTimes();
	Keys.Reset();
	CachedSelection.Reset();
	if (OwningTrail->GetChannelOffset() == INDEX_NONE)
	{
		return;
	}
	UMovieSceneSection* AbsoluteTransformSection = OwningTrail->GetSection();
	int32 MaxChannel = (int32)ETransformChannel::TranslateZ;//only do the first three channels, 0,1,2 which are position.
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = AbsoluteTransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	if (DoubleChannels.Num() > MaxChannel)
	{
		DoubleChannels = DoubleChannels.Slice(OwningTrail->GetChannelOffset(), MaxChannel + 1);
		for (int32 ChannelIdx = 0; ChannelIdx <= MaxChannel; ChannelIdx++)
		{
			FMovieSceneDoubleChannel* DoubleChannel = DoubleChannels[ChannelIdx];
			for (int32 Idx = 0; Idx < DoubleChannel->GetNumKeys(); Idx++)
			{
				FFrameNumber CurTime = DoubleChannel->GetTimes()[Idx];

				if (!Keys.Contains(CurTime))
				{
					TUniquePtr<FTrailKeyInfo> TempKeyInfo = MakeUnique<FTrailKeyInfo>(CurTime, AbsoluteTransformSection, OwningTrail);
					Keys.Add(MoveTemp(CurTime), MoveTemp(TempKeyInfo));
				}
			}
		}
	}
	else
	{
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = AbsoluteTransformSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		if (FloatChannels.Num() > MaxChannel)
		{
			FloatChannels = FloatChannels.Slice(OwningTrail->GetChannelOffset(), MaxChannel + 1);
			for (int32 ChannelIdx = 0; ChannelIdx <= MaxChannel; ChannelIdx++)
			{
				FMovieSceneFloatChannel* FloatChannel = FloatChannels[ChannelIdx];
				for (int32 Idx = 0; Idx < FloatChannel->GetNumKeys(); Idx++)
				{
					FFrameNumber CurTime = FloatChannel->GetTimes()[Idx];

					if (!Keys.Contains(CurTime))
					{
						TUniquePtr<FTrailKeyInfo> TempKeyInfo = MakeUnique<FTrailKeyInfo>(CurTime, AbsoluteTransformSection, OwningTrail);
						Keys.Add(MoveTemp(CurTime), MoveTemp(TempKeyInfo));
					}
				}
			}
		}
	}
	if (KeyTimes.Num() > 0)
	{
		SelectKeyTimes(KeyTimes,false);
	}
}

TArray<FKeyHandle> FMotionTraiMovieScenelKeyTool::GetSelectedKeyHandles(FMovieSceneChannel *Channel) 
{
	TArray<FKeyHandle> TotalKeyHandles;
	for (const FTrailKeyInfo* KeyInfo : CachedSelection)
	{
		TRange<FFrameNumber> FrameRange(KeyInfo->FrameNumber, KeyInfo->FrameNumber);
		TArray<FKeyHandle> KeyHandles;
		TArray<FFrameNumber> KeyTimes;
		Channel->GetKeys(FrameRange, &KeyTimes, &KeyHandles);
		for (FKeyHandle& Handle : KeyHandles)
		{
			TotalKeyHandles.Add(Handle);
		}
	}
	return TotalKeyHandles;
}

void FMotionTraiMovieScenelKeyTool::TranslateSelectedKeys(bool bRight)
{
	if (OwningTrail->GetChannelOffset() == INDEX_NONE)
	{
		return;
	}
	UMovieSceneSection* Section = OwningTrail->GetSection();
	if (CachedSelection.Num() > 0 && Section && Section->TryModify())
	{
		Section->Modify();
		int32 Shift = bRight ? 1 : -1;
		FFrameNumber Delta = FQualifiedFrameTime(Shift, OwningTrail->GetSequencer()->GetFocusedDisplayRate()).ConvertTo(OwningTrail->GetSequencer()->GetFocusedTickResolution()).RoundToFrame();
		TRange<FFrameNumber> SectionNewBounds = Section->GetRange();
		TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		bool bIsFloat = false;
		int32 MaxChannel = (int32)ETransformChannel::TranslateZ;//only do the first three channels, 0,1,2 which are position.
		if (DoubleChannels.Num() > MaxChannel)
		{
			bIsFloat = false;
			DoubleChannels = DoubleChannels.Slice(OwningTrail->GetChannelOffset(), MaxChannel + 1);
		}
		else if (FloatChannels.Num() > MaxChannel)
		{
			bIsFloat = true;
			FloatChannels = FloatChannels.Slice(OwningTrail->GetChannelOffset(), MaxChannel + 1);
		}
		else
		{
			return;
		}
		for (int32 ChannelIdx = 0; ChannelIdx <= MaxChannel; ChannelIdx++)
		{
			TArray<FFrameNumber> KeyTimes;
			FMovieSceneChannel* Channel = nullptr;
			if (bIsFloat)
			{
				Channel = FloatChannels[ChannelIdx];
			}
			else
			{
				Channel = DoubleChannels[ChannelIdx];
			}
			TArray<FKeyHandle> KeyHandles = GetSelectedKeyHandles(Channel);
			if (KeyHandles.Num() > 0)
			{
				KeyTimes.SetNum(KeyHandles.Num());
				Channel->GetKeyTimes(KeyHandles, KeyTimes);
				FFrameNumber LowestFrameTime = KeyTimes[0];
				FFrameNumber HighestFrameTime = KeyTimes[0];

				// Perform the transformation
				for (FFrameNumber& Time : KeyTimes)
				{
					FFrameTime KeyTime = Time;
					Time += Delta;
					if (Time < LowestFrameTime)
					{
						LowestFrameTime = Time;
					}
					if (Time > HighestFrameTime)
					{
						HighestFrameTime = Time;
					}
				}
				SectionNewBounds = TRange<FFrameNumber>::Hull(SectionNewBounds, TRange<FFrameNumber>(LowestFrameTime, HighestFrameTime + 1));
				Section->SetRange(SectionNewBounds);
				Channel->SetKeyTimes(KeyHandles, KeyTimes);
			}

		}

		for (FTrailKeyInfo* KeyInfo : CachedSelection)
		{
			KeyInfo->FrameNumber += Delta;
		}
		BuildKeys();

		OwningTrail->GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		UpdateSelectedKeysTransform();
	}
}

void FMotionTraiMovieScenelKeyTool::DeleteSelectedKeys()
{
	if (OwningTrail->GetChannelOffset() == INDEX_NONE)
	{
		return;
	}
	UMovieSceneSection* Section = OwningTrail->GetSection();
	if (CachedSelection.Num() > 0 && Section && Section->TryModify())
	{
		Section->Modify();
		bool bIsFloat = true;
		int32 MaxChannel = (int32)ETransformChannel::TranslateZ;///only do the first three channels, 0,1,2 which are position.
		TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

		if (DoubleChannels.Num() > 0)
		{
			DoubleChannels = DoubleChannels.Slice(OwningTrail->GetChannelOffset(), MaxChannel + 1);
			bIsFloat = false;
		}
		else if (FloatChannels.Num() > 0)
		{
			FloatChannels = FloatChannels.Slice(OwningTrail->GetChannelOffset(), MaxChannel + 1);
			bIsFloat = true;
		}
		else
		{
			return;
		}
		for (int32 ChannelIdx = 0; ChannelIdx <=  MaxChannel; ChannelIdx++)
		{
			FMovieSceneChannel* Channel = nullptr;
			if (bIsFloat)
			{
				Channel = FloatChannels[ChannelIdx];
			}
			else
			{
				Channel = DoubleChannels[ChannelIdx];
			}
			TArray<FKeyHandle> KeyHandles = GetSelectedKeyHandles(Channel);
			Channel->DeleteKeys(KeyHandles);
		}
		OwningTrail->GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		CachedSelection.Reset();
		UpdateSelectedKeysTransform();
	}
}

FTrailKeyInfo* FMotionTraiMovieScenelKeyTool::FindKey(const FFrameNumber& FrameNumber) const
{
	const TUniquePtr<FTrailKeyInfo>* KeyInfo = Keys.Find(FrameNumber);
	if (KeyInfo != nullptr)
	{
		return (KeyInfo->Get());
	}
	//due to seconds to frame issues also try previous and next frames
	KeyInfo = Keys.Find(FrameNumber + 1);
	if (KeyInfo != nullptr)
	{
		return (KeyInfo->Get());
	}
	KeyInfo = Keys.Find(FrameNumber -1);
	if (KeyInfo != nullptr)
	{
		return (KeyInfo->Get());
	}

	return nullptr;
}

TArray<FFrameNumber> FMotionTraiMovieScenelKeyTool::SelectedKeyTimes() const
{
	TArray<FFrameNumber> Frames;
	for (const FTrailKeyInfo* KeyInfo : CachedSelection)
	{
		Frames.Add(KeyInfo->FrameNumber);
	}
	return Frames;
}

void FMotionTraiMovieScenelKeyTool::SelectKeyTimes(const TArray<FFrameNumber>& Frames, bool KeepSelection)
{
	if (KeepSelection == false)
	{
		CachedSelection.Reset();
	}
	for (const FFrameNumber& FrameNumber : Frames)
	{
		FTrailKeyInfo* KeyInfo = FindKey(FrameNumber);
		if (KeyInfo)
		{
			CachedSelection.Add(KeyInfo);
		}
	}
}

TArray<FFrameNumber> FMotionTraiMovieScenelKeyTool::GetTimesFromModifiedTimes(const TArray<FFrameNumber>& ModifiedFrames, const FFrameNumber& LastFrame, const FFrameNumber& Step)
{
	TArray<FFrameNumber> TotalModifiedFrames;
	TSet<int32> Indices;
	bool bAddLastKey = false; //if we modified the last key we need to go all the way to the end
	//first we calculate the indices that have changed, we get the two before and the one after
	for (const FFrameNumber& FrameNumber : ModifiedFrames)
	{
		int32 Index = Keys.FindIndex(FrameNumber);
		if (Index == INDEX_NONE)  //something went wrong pass out empty array which means we will recalc everything
		{
			return TotalModifiedFrames;
		}
		else
		{
			if (Index - 2 >= 0)
			{
				if (Indices.Contains(Index - 2) == false)
				{
					Indices.Add(Index - 2);
				}
			}
			if (Index - 1 >= 0)
			{
				if (Indices.Contains(Index - 1) == false)
				{
					Indices.Add(Index - 1);
				}
			}
			if (Indices.Contains(Index) == false)
			{
				Indices.Add(Index);
			}
			if (Index + 1 < Keys.Num())
			{
				if (Indices.Contains(Index + 1) == false)
				{
					Indices.Add(Index + 1);
				}
				if (Index + 2 == Keys.Num())
				{
					bAddLastKey = true;
				}
			}
			else
			{
				bAddLastKey = true;
			}
		}
	}
	for (int32 Index : Indices)
	{
		TUniquePtr<FTrailKeyInfo>* FirstKeyInfo =  Keys.FindFromIndex(Index);
		TUniquePtr<FTrailKeyInfo>* SecondKeyInfo = Keys.FindFromIndex(Index + 1);
		if (FirstKeyInfo && FirstKeyInfo->IsValid() && SecondKeyInfo && SecondKeyInfo->IsValid())
		{
			for (FFrameNumber Frame = FirstKeyInfo->Get()->FrameNumber; Frame < SecondKeyInfo->Get()->FrameNumber; Frame += Step)
			{
				TotalModifiedFrames.Add(Frame);
			}
			
		}
	}
	if (bAddLastKey)
	{
		for (FFrameNumber Frame = TotalModifiedFrames[TotalModifiedFrames.Num() - 1]; Frame <= LastFrame; Frame += Step)
		{
			TotalModifiedFrames.Add(Frame);
		}
	}
	return TotalModifiedFrames;
}

bool FMotionTraiMovieScenelKeyTool::ShouldRebuildKeys()
{
	if (OwningTrail->GetChannelOffset() == INDEX_NONE)
	{
		return false;
	}
	TMap<FFrameNumber, TSet<ETransformChannel>> KeyTimes;
	int32 MaxChannel = (int32)ETransformChannel::TranslateZ;///only do the first three channels, 0,1,2 which are position.
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = OwningTrail->GetSection()->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = OwningTrail->GetSection()->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	if (DoubleChannels.Num() > MaxChannel)
	{
		DoubleChannels = DoubleChannels.Slice(OwningTrail->GetChannelOffset(), MaxChannel + 1);
		for (int32 ChannelIdx = 0; ChannelIdx <= MaxChannel; ChannelIdx++)
		{
			FMovieSceneDoubleChannel* DoubleChannel = DoubleChannels[ChannelIdx];
			for (int32 Idx = 0; Idx < DoubleChannel->GetNumKeys(); Idx++)
			{
				const FFrameNumber CurTime = DoubleChannel->GetTimes()[Idx];
				KeyTimes.FindOrAdd(CurTime).Add(ETransformChannel(ChannelIdx));
			}
		}
	}
	else if (FloatChannels.Num() > MaxChannel)
	{
		FloatChannels = FloatChannels.Slice(OwningTrail->GetChannelOffset(), MaxChannel + 1);
		for (int32 ChannelIdx = 0; ChannelIdx <= MaxChannel; ChannelIdx++)
		{
			FMovieSceneFloatChannel* FloatChannel = FloatChannels[ChannelIdx];
			for (int32 Idx = 0; Idx < FloatChannel->GetNumKeys(); Idx++)
			{
				const FFrameNumber CurTime = FloatChannel->GetTimes()[Idx];
				KeyTimes.FindOrAdd(CurTime).Add(ETransformChannel(ChannelIdx));
			}
		}
	}

	if (KeyTimes.Num() != Keys.Num())
	{
		return true;
	}

	for (const TPair<FFrameNumber, TSet<ETransformChannel>>& TimeKeyPair : KeyTimes)
	{
		if (!Keys.Contains(TimeKeyPair.Key))
		{
			return true;
		}
	}
	return false;
}

void FMotionTraiMovieScenelKeyTool::ClearSelection()
{
	CachedSelection.Reset();
}

void FMotionTraiMovieScenelKeyTool::DirtyKeyTransforms()
{
	for (const TPair<FFrameNumber, TUniquePtr<FTrailKeyInfo>>& FrameKeyPair : Keys.Keys)
	{
		FrameKeyPair.Value->bDirty = true;
	}
}
void FMotionTraiMovieScenelKeyTool::UpdateViewRange(const TRange<FFrameNumber>& InViewRange)
{
	CachedViewFrameRange = InViewRange;
}

void FMotionTraiMovieScenelKeyTool::UpdateKeys()
{
	for (const TPair<FFrameNumber, TUniquePtr<FTrailKeyInfo>>& FrameKeyPair : Keys.Keys)
	{
		if (FrameKeyPair.Value->bDirty)
		{
			FrameKeyPair.Value->UpdateKeyTransform(EGetKeyFrom::FromTrailCache);
		}
	}
	UpdateSelectedKeysTransform();
}

FTrailKeyInfo::FTrailKeyInfo(const FFrameNumber InFrameNumber, UMovieSceneSection* InSection, FMovieSceneTransformTrail* InOwningTrail):
  IdxMap()
, FrameNumber(InFrameNumber)
, bDirty(true)
, OwningTrail(InOwningTrail)
{
	if (OwningTrail->GetChannelOffset() == INDEX_NONE)
	{
		return;
	}
	int32 MaxChannel = (int32)ETransformChannel::TranslateZ;///only do the first three channels, 0,1,2 which are position.
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	if (DoubleChannels.Num() > MaxChannel)
	{
		DoubleChannels = DoubleChannels.Slice(OwningTrail->GetChannelOffset(),  MaxChannel + 1);
		for (int32 Idx = 0; Idx <= MaxChannel; Idx++)
		{
			const int32 FoundIdx = DoubleChannels[Idx]->GetData().FindKey(InFrameNumber);
			if (FoundIdx != INDEX_NONE)
			{
				IdxMap.Add(ETransformChannel(Idx), DoubleChannels[Idx]->GetData().GetHandle(FoundIdx));
			}
		}
	}
	else if (FloatChannels.Num() > MaxChannel)
	{
		FloatChannels = FloatChannels.Slice(OwningTrail->GetChannelOffset(), MaxChannel + 1);
		for (int32 Idx = 0; Idx <= MaxChannel; Idx++)
		{
			const int32 FoundIdx = FloatChannels[Idx]->GetData().FindKey(InFrameNumber);
			if (FoundIdx != INDEX_NONE)
			{
				IdxMap.Add(ETransformChannel(Idx), FloatChannels[Idx]->GetData().GetHandle(FoundIdx));
			}
		}
	}
	
}

void FTrailKeyInfo::UpdateKeyTransform(EGetKeyFrom UpdateType)
{	
	bDirty = false;
	if (UpdateType == EGetKeyFrom::FromTrailCache)
	{
		 OwningTrail->Interp(FrameNumber, Transform, ParentTransform);
	}
}

} // namespace MovieScene
} // namespace UE

