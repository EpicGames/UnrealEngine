// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/KeyHandle.h"
#include "Misc/FrameNumber.h"

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class HHitProxy;
namespace UE::SequencerAnimTools { struct FInputClick; }

class UMovieSceneSection;
struct FMovieSceneChannel;

namespace UE
{
namespace SequencerAnimTools
{


class FMovieSceneTransformTrail;
class FMovieSceneControlRigTransformTrail;
class FMovieSceneComponentTransformTrail;

enum class ETransformChannel : uint8
{
	TranslateX = 0,
	TranslateY = 1,
	TranslateZ = 2,
	RotateX = 3,
	RotateY = 4,
	RotateZ = 5,
	ScaleX = 6,
	ScaleY = 7,
	ScaleZ = 8,
	MaxChannel = 8
};

enum class EGetKeyFrom
{
	FromComponentDelta,
	FromTrailCache
};

struct FTrailKeyInfo
{
	FTrailKeyInfo(const FFrameNumber InFrameNumber, UMovieSceneSection* InSection,
		FMovieSceneTransformTrail* InOwningTrail);

	// Re-eval transform or use given one
	void UpdateKeyTransform(EGetKeyFrom UpdateType);

	// Key Specific info
	FTransform Transform;
	FTransform ParentTransform;
	TMap<ETransformChannel, FKeyHandle> IdxMap;
	FFrameNumber FrameNumber;
	bool bDirty;

	FMovieSceneTransformTrail* OwningTrail;
};

//This class is a limtited version of TSortedMap that also exposes FindIndex and a new FindFromIndex function
//in order to get direct access to the hidden sorted array.
class FSortedKeys
{
	struct FKeyForward
	{
		FORCEINLINE const FFrameNumber& operator()(const TPair<FFrameNumber, TUniquePtr<FTrailKeyInfo>>& Pair) const
		{
			return Pair.Key;
		}
	};
public:
	int32 FindIndex(FFrameNumber Key) const
	{
		return Algo::BinarySearchBy(Keys, Key, FKeyForward());
	}

	const TUniquePtr<FTrailKeyInfo>* Find(FFrameNumber Key) const
	{
		int32 Index = FindIndex(Key);
		if (Index == INDEX_NONE)
		{
			return nullptr;
		}
		else return &(Keys[Index].Value);
	}

	int32 FindIndex(FFrameNumber Key) 
	{
		return Algo::BinarySearchBy(Keys, Key, FKeyForward());
	}
	TUniquePtr<FTrailKeyInfo>* FindFromIndex(int32 Index)
	{
		if (Index >= 0 && Index < Keys.Num())
		{
			return &(Keys[Index].Value);
		}
		return nullptr;
	}
	TUniquePtr<FTrailKeyInfo>* Find(FFrameNumber Key) 
	{
		int32 Index = FindIndex(Key);
		if (Index == INDEX_NONE)
		{
			return nullptr;
		}
		else return &(Keys[Index].Value);
	}
	bool Contains(FFrameNumber Key) const
	{
		if (Find(Key))
		{
			return true;
		}
		return false;
	}

	void Empty(int32 ExpectedNumElements = 0)
	{
		Keys.Empty(ExpectedNumElements);
	}

	void Reset()
	{
		Keys.Reset();
	}

	void Shrink()
	{
		Keys.Shrink();
	}

	void Reserve(int32 Number)
	{
		Keys.Reserve(Number);
	}

	bool IsEmpty() const
	{
		return Keys.IsEmpty();
	}

	int32 Num() const
	{
		return Keys.Num();
	}

	TUniquePtr<FTrailKeyInfo>& FindChecked(FFrameNumber Key)
	{
		TUniquePtr<FTrailKeyInfo>* Value = Find(Key);
		check(Value != nullptr);
		return *Value;
	}

	const TUniquePtr<FTrailKeyInfo>& FindChecked(FFrameNumber Key) const
	{
		const TUniquePtr<FTrailKeyInfo>* Value = Find(Key);
		check(Value != nullptr);
		return *Value;
	}

	TUniquePtr<FTrailKeyInfo>& operator[](FFrameNumber Key) 
	{ 
		return FindChecked(Key);
	}
	const TUniquePtr<FTrailKeyInfo>& operator[](FFrameNumber Key) const
	{ 
		return FindChecked(Key); 
	}
	
	FORCEINLINE TPair<FFrameNumber, TUniquePtr<FTrailKeyInfo>>* AllocateMemoryForEmplace(FFrameNumber&& InKey)
	{
		int32 InsertIndex = Algo::LowerBoundBy(Keys, InKey, FKeyForward());
		check(InsertIndex >= 0 && InsertIndex <= Keys.Num());

		TPair<FFrameNumber, TUniquePtr<FTrailKeyInfo>>* DataPtr = nullptr;
		// Adding new one, this may reallocate Pairs
		Keys.InsertUninitialized(InsertIndex, 1);
		DataPtr = Keys.GetData() + InsertIndex;
		return DataPtr;
	}
	
	TUniquePtr<FTrailKeyInfo>& Add(FFrameNumber &&InKey, TUniquePtr<FTrailKeyInfo> && InValue)
	{
		TPair<FFrameNumber, TUniquePtr<FTrailKeyInfo>>* DataPtr = AllocateMemoryForEmplace(MoveTemp(InKey));

		new(DataPtr) TPair<FFrameNumber, TUniquePtr<FTrailKeyInfo>>(TPairInitializer<FFrameNumber&&, TUniquePtr<FTrailKeyInfo>&&>(Forward<FFrameNumber>(InKey), Forward< TUniquePtr<FTrailKeyInfo>>(InValue)));

		return DataPtr->Value;
	}
	
	void GetAllKeys(TArray<FTrailKeyInfo*>& OutKeys)
	{
		OutKeys.SetNum(Keys.Num());
		int32 Index = 0;
		for (TPair < FFrameNumber, TUniquePtr<FTrailKeyInfo>>& Pair : Keys)
		{
			OutKeys[Index++] = Pair.Value.Get();
		}
	}

	TArray < TPair<FFrameNumber, TUniquePtr<FTrailKeyInfo>> > Keys;
};

class FMotionTraiMovieScenelKeyTool 
{
public:
	FMotionTraiMovieScenelKeyTool(FMovieSceneTransformTrail* InOwningTrail)
		: OwningTrail(InOwningTrail), SelectedKeysTransform(FTransform::Identity)
	{}
	~FMotionTraiMovieScenelKeyTool() = default;

	void Render(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, bool bTrailIsEvaluating);
	void DrawHUD(const FSceneView* View, FCanvas* Canvas);

	bool HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, FInputClick Click);
	void StartDragging() {};
	bool IsSelected(FVector& OutVectorPosition, FQuat& OutRotation) const;
	void GetSelectedKeyPositions(TArray<FVector>& OutPositions) const;
	bool IsSelected() const;
	void OnSectionChanged();
	void UpdateKeys();
	void UpdateViewRange(const TRange<FFrameNumber>& InViewRange);
	void ClearSelection();
	FTrailKeyInfo* FindKey(const FFrameNumber& FrameNumber) const;
	TArray<FFrameNumber> SelectedKeyTimes() const;
	void SelectKeyTimes(const TArray<FFrameNumber>& Frames, bool KeepSelection);
	void SelectKeyInfo(FTrailKeyInfo* KeyInfo) { CachedSelection.Add(KeyInfo); }
	FTransform GetSelectedKeysTransform() const { return SelectedKeysTransform; }
	void TranslateSelectedKeys(bool bRight);
	void DeleteSelectedKeys();
	bool IsAllSelected() const { return Keys.Num() == CachedSelection.Num(); }
	void GetAllKeys(TArray<FTrailKeyInfo*>& OutKeys) {Keys.GetAllKeys(OutKeys); }
	const FSortedKeys& GetSortedKeys()const { return Keys; }
	TArray<FFrameNumber> GetTimesFromModifiedTimes(const TArray<FFrameNumber>& Frames, const FFrameNumber& LastFrame, const FFrameNumber& Step);
protected:
	TRange<FFrameNumber> CachedViewFrameRange;
	void UpdateSelectedKeysTransform() const;
	bool ShouldRebuildKeys();
	void BuildKeys();
	void DirtyKeyTransforms();
	TArray<FKeyHandle> GetSelectedKeyHandles(FMovieSceneChannel* Channel);

	FSortedKeys Keys;
	TSet<FTrailKeyInfo*> CachedSelection; //current selection

	FMovieSceneTransformTrail* OwningTrail;
	mutable FTransform SelectedKeysTransform;

	friend class FMovieSceneTransformTrail;
	friend class FMovieSceneComponentTransformTrail;
	friend class FMovieSceneControlRigTransformTrail;
};


} // namespace MovieScene
} // namespace UE
