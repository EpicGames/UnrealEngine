// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTransformTrail.h"
#include "Engine/Engine.h"
#include "TrailHierarchy.h"
#include "SequencerTrailHierarchy.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Tracks/MovieScene3DTransformTrack.h"

#include "ControlRigObjectBinding.h"
#include "Tools/MotionTrailOptions.h"

#include "MovieSceneObjectBindingID.h"
#include "MovieSceneToolHelpers.h"
#include "ActorForWorldTransforms.h"
#include "Evaluation/MovieScenePlayback.h"
#include "ScopedTransaction.h"
#include "Sequencer/MotionTrailMovieSceneKey.h"
#include "Framework/Application/SlateApplication.h"
#include "PrimitiveDrawInterface.h"
#include "Components/SkeletalMeshComponent.h"
#include "EditMode/ControlRigEditMode.h"
#include "UnrealEdGlobals.h"
#include "EditorModeManager.h"


#define LOCTEXT_NAMESPACE "SequencerAnimTools"

namespace UE
{
namespace SequencerAnimTools
{

using namespace UE::AIE;


/****************************************************************************************************************
*
*   Optional Parent Space
*
*****************************************************************************************************************/

void FOptionalParentSpace::SetSpace(TSharedPtr<ISequencer>&  InSequencer, AActor* InActor, const FName& InComponentName)
{
	if (InActor)
	{
		bIsValid = true;
		ComponentName = NAME_None; 
		ParentSpace.Actor.Actor = InActor;
		if (InComponentName != NAME_None)
		{
			if (USceneComponent* Component = GetComponentFromName(InActor, InComponentName))
			{
				ParentSpace.Actor.Component = Component;
				ComponentName = InComponentName;
			}
		}

		if (ParentSpace.WorldTransforms.IsValid() == false)
		{
			ParentSpace.WorldTransforms = MakeShared<FArrayOfTransforms>();
		}
		if (ParentSpace.ParentTransforms.IsValid() == false)
		{
			ParentSpace.ParentTransforms = MakeShared<FArrayOfTransforms>();
		}

		check(InSequencer);
		TArray<FMovieSceneObjectBindingID> Bindings;
		FMovieSceneEvaluationState* State = InSequencer->GetEvaluationState();
		State->FilterObjectBindings(ParentSpace.Actor.Component.Get(), *InSequencer, &Bindings);
		if (Bindings.Num() <= 0)
		{
			AActor* ParentActor = ParentSpace.Actor.Component->GetOwner();
			State->FilterObjectBindings(ParentActor, *InSequencer, &Bindings);
		}
		if (Bindings.Num() > 0)
		{
			SpaceBindingID = Bindings[0].GetGuid();
		}
	}
}

USceneComponent* FOptionalParentSpace::GetComponentFromName(const AActor* InActor, const FName& InComponentName)
{
	TInlineComponentArray<USceneComponent*> Components(InActor);
	for (USceneComponent* Component : Components)
	{
		if (Component->GetFName() == InComponentName)
		{
			return Component;
		}
	}
	return nullptr;
}

void FOptionalParentSpace::ClearSpace()
{
	bIsValid = false;
	ParentSpace.Actor.Actor = nullptr;
	SpaceBindingID = FGuid();
	ComponentName = NAME_None;
}

/****************************************************************************************************************
*
*   Base Movie Scene Transform Trail
*
*****************************************************************************************************************/


FMovieSceneTransformTrail::FMovieSceneTransformTrail(const FGuid& InBindingID, USceneComponent* InOwner, const bool bInIsVisible, TWeakObjectPtr<UMovieSceneTrack> InWeakTrack, TSharedPtr<ISequencer> InSequencer)
	: FTrail(InOwner)
	, WeakSequencer(InSequencer)
	, bIsSelected(false)
	, bIsOffsetMode(false)
	, bIsTracking(false)
	, BindingID(InBindingID)
	, MainTrack(InWeakTrack)
{
	KeyTool = MakeUnique< FMotionTraiMovieScenelKeyTool>(this);

	//new
	CalculatedArrayOfTransforms = MakeShared<FArrayOfTransforms>();
	ArrayOfTransforms = MakeShared<FArrayOfTransforms>();
	ParentArrayOfTransforms = MakeShared<FArrayOfTransforms>();

	DrawInfo = MakeUnique<FTrajectoryDrawInfo>(UMotionTrailToolOptions::GetTrailOptions()->TrailStyle, UMotionTrailToolOptions::GetTrailOptions()->DefaultColor, ArrayOfTransforms, ParentSpace.ParentSpace.WorldTransforms);
}

FMovieSceneTransformTrail::~FMovieSceneTransformTrail()
{
	ExitOffsetMode();
}

void FMovieSceneTransformTrail::HasStartedEvaluating()
{
	PreviousCachedDrawData = CachedDrawData;
}

void FMovieSceneTransformTrail::UpdateFinished(const TRange<FFrameNumber>& Range, const TArray<int32>& IndicesToCalcluate, bool bDoneCalculating)
{
	FTrail::UpdateFinished(Range, IndicesToCalcluate, bDoneCalculating);
	if (HasOffsetTransform() == false)
	{
		for (int32 Count = 0; Count < IndicesToCalcluate.Num(); ++Count)
		{
			int32 Index = IndicesToCalcluate[Count];
			ArrayOfTransforms->Transforms[Index] = CalculatedArrayOfTransforms->Transforms[Index];
		}
	}
	else
	{
		TSet<int32> CalculatedIndices;
		for (int32 Count = 0; Count < IndicesToCalcluate.Num(); ++Count)
		{
			int32 Index = IndicesToCalcluate[Count];
			if (CalculatedIndices.Contains(Index) == false)
			{
				CalculatedIndices.Add(Index);
				ArrayOfTransforms->Transforms[Index] = OffsetTransform * CalculatedArrayOfTransforms->Transforms[Index];
			}
		}
	}
	//if we are in a parent space we need to put transforms in that space
	if (ParentSpace.bIsValid && IndicesToCalcluate.Num() > 0)
	{
		TSet<int32> CalculatedIndices;
		for (int32 Count = 0; Count < IndicesToCalcluate.Num(); ++Count)
		{
			int32 Index = IndicesToCalcluate[Count];
			if (CalculatedIndices.Contains(Index) == false)
			{
				CalculatedIndices.Add(Index);
				ArrayOfTransforms->Transforms[Index] = ArrayOfTransforms->Transforms[Index].GetRelativeTransform(ParentSpace.ParentSpace.WorldTransforms->Transforms[Index]);
				ParentArrayOfTransforms->Transforms[Index] = ParentArrayOfTransforms->Transforms[Index].GetRelativeTransform(ParentSpace.ParentSpace.WorldTransforms->Transforms[Index]);
			}
		}

	}
	if (bDoneCalculating)
	{
		bIsTracking = false;
	}
	if (bIsTracking == false)
	{
		KeyTool->DirtyKeyTransforms();
		KeyTool->UpdateKeys();
	}
}
TArray<UObject*> FMovieSceneTransformTrail::GetBoundObjects(TSharedPtr<ISequencer>& InSequencerPtr, const FGuid& InGuid)
{
	TArray<UObject*> BoundObjects;
	if (InSequencerPtr.IsValid())
	{
		FMovieSceneSequenceID SequenceId = InSequencerPtr->GetEvaluationState()->FindSequenceId(InSequencerPtr->GetFocusedMovieSceneSequence());
		FMovieSceneObjectBindingID ObjectBinding = UE::MovieScene::FFixedObjectBindingID(InGuid, SequenceId);

		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(InSequencerPtr->GetFocusedTemplateID(), *InSequencerPtr))
		{
			if (WeakObject.IsValid())
			{
				BoundObjects.Add(WeakObject.Get());
			}
		}
	}
	return BoundObjects;
}
//we need a sequencer for this so FTrail does not have an implementation
void FMovieSceneTransformTrail::CheckAndUpdateObjects()
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencer())
	{
		TArray<UObject*> BoundObjects;
		if (Owner.IsValid() == false)
		{
			BoundObjects = GetBoundObjects(SequencerPtr, BindingID);
			if (BoundObjects.Num() > 0)
			{
				if (AActor* Actor = Cast<AActor>(BoundObjects[0]))
				{
					Owner = Actor->GetRootComponent();
				}
				else if (USceneComponent* Comp = Cast<USceneComponent>(BoundObjects[0]))
				{
					Owner = Comp;
				}
			}
			BoundObjects.SetNum(0);
		}
		if (ParentSpace.bIsValid && ( (ParentSpace.ParentSpace.Actor.Actor.IsValid() == false)
			|| (ParentSpace.ComponentName.IsNone() == false && ParentSpace.ParentSpace.Actor.Component.IsValid() == false) ))
		{
			BoundObjects = GetBoundObjects(SequencerPtr, ParentSpace.SpaceBindingID);
			if (BoundObjects.Num() > 0)
			{
				if (AActor* Actor = Cast<AActor>(BoundObjects[0]))
				{
					ParentSpace.ParentSpace.Actor.Actor = Actor;
					if (ParentSpace.ComponentName.IsNone() == false)
					{
						USceneComponent* Component = FOptionalParentSpace::GetComponentFromName(Actor, ParentSpace.ComponentName);
						ParentSpace.ParentSpace.Actor.Component = Component;
					}
				}
				else if (USceneComponent* Comp = Cast<USceneComponent>(BoundObjects[0]))
				{
					ParentSpace.ParentSpace.Actor.Component = Comp;
					ParentSpace.ParentSpace.Actor.Actor = Comp->GetOwner();
				}
			}
		}
	}
}

bool FMovieSceneTransformTrail::HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap) 
{
	bool bSomethingChanged = FTrail::HandleObjectsChanged(ReplacementMap);
	if (ParentSpace.bIsValid)
	{
		if (UObject* const* NewObject = ReplacementMap.Find(ParentSpace.ParentSpace.Actor.Component.Get()))
		{
			ParentSpace.ParentSpace.Actor.Component = Cast<USceneComponent>(*NewObject);
			bSomethingChanged = bSomethingChanged || true;
		}
		if (UObject* const* NewObject = ReplacementMap.Find(ParentSpace.ParentSpace.Actor.Actor.Get()))
		{
			ParentSpace.ParentSpace.Actor.Actor = Cast<AActor>(*NewObject);
			bSomethingChanged = bSomethingChanged || true;
		}
	}
	return bSomethingChanged;
}

FTransform FMovieSceneTransformTrail::GetParentSpaceTransform()  const
{
	if (ParentSpace.bIsValid == true)
	{
		if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
		{
			FFrameTime Time = Sequencer->GetLocalTime().Time;
			auto GetParentComponent = [this]() -> USceneComponent*
			{
				if (ParentSpace.ParentSpace.Actor.Component.IsValid())
				{
					return  ParentSpace.ParentSpace.Actor.Component.Get();
				}
				else if (ParentSpace.ParentSpace.Actor.Actor.IsValid())
				{
					return ParentSpace.ParentSpace.Actor.Actor->GetRootComponent();
				}
				return nullptr;
			};

			USceneComponent* ParentComponent = GetParentComponent();
			
			FTransform ParentSpaceTM = ParentComponent ? ParentComponent->GetComponentToWorld() : FTransform::Identity;				
			if (PreviousParentSpaceTM.Equals(ParentSpaceTM) == false)
			{
				CachedDrawData.PointsToDraw.SetNum(0);
				PreviousParentSpaceTM = ParentSpaceTM;
			}
			return ParentSpaceTM;
		}
	}
	return FTransform::Identity;
}

void FMovieSceneTransformTrail::AddImportantTimes(TSet<FFrameNumber>& InOutImportantTimes)
{
	TArray<FFrameNumber> SelectedKeyTimes = KeyTool->SelectedKeyTimes();
	{
		for (const FFrameNumber& FrameNumber : SelectedKeyTimes)
		{
			InOutImportantTimes.Add(FrameNumber);
		}
	}
}

UMovieSceneSection* FMovieSceneTransformTrail::GetSection() const
{
	UMovieSceneSection* Section = nullptr;
	if (MainTrack.IsValid())
	{
		Section = MainTrack->GetSectionToKey();
		if (!Section && MainTrack->GetAllSections().Num() > 0)
		{
			Section = MainTrack->GetAllSections()[0];
		}
	}
	return Section;
}

void FMovieSceneTransformTrail::ForceEvaluateNextTick()
{
	FTrail::ForceEvaluateNextTick();
	KeyTool->DirtyKeyTransforms();
	ClearCachedData();
}

//if actor is not a sequencer actor then it's static so we need to update the trails
void FMovieSceneTransformTrail::ActorChanged(AActor* InActor)
{
	if (TransformDependencies.NonSequencerActors.Contains(InActor))
	{
		ForceEvaluateNextTick();
	}
}

bool FMovieSceneTransformTrail::BindingHasChanged(const FGuid& InBindingID, const USceneComponent* InComponent, UE::AIE::FSequencerTransformDependencies& InOutDependencies) 
{
	if(TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
	{ 
		if (InComponent)
		{
			UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
			if (FMovieSceneBinding* Binding = MovieScene->FindBinding(InBindingID))
			{
				AActor* Actor = InComponent->GetOwner();
				UE::AIE::FSequencerTransformDependencies NewTransformDependencies;
				TArray<UMovieSceneTrack*> Tracks = FSequencerTransformDependencies::GetTransformAffectingTracks(MovieScene,
					*Binding);

				NewTransformDependencies.CalculateDependencies(WeakSequencer.Pin().Get(), Actor, Tracks);
				if (InOutDependencies.Compare(NewTransformDependencies) == false)
				{
					InOutDependencies.CopyFrom(NewTransformDependencies);
					return true;
				}
			}
		}
	}
	return false;
}
void FMovieSceneTransformTrail::SetSpace(AActor* InActor, const FName& InComponentName)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	ParentSpace.SetSpace(SequencerPtr, InActor, InComponentName);
}

bool FMovieSceneTransformTrail::TrailOrSpaceHasChanged() 
{
	if (USceneComponent* Component = Cast<USceneComponent>(GetOwner()))
	{
		if (BindingHasChanged(BindingID, Component, TransformDependencies))
		{
			return true;
		}
	}

	if (ParentSpace.bIsValid == true)
	{
		if (BindingHasChanged(ParentSpace.SpaceBindingID, ParentSpace.ParentSpace.Actor.Component.Get(), TransformDependencies))
		{
			return true;
		}
	}
	return false;
}

FTrailCurrentStatus FMovieSceneTransformTrail::UpdateTrail(const FNewSceneContext& InSceneContext)
{

	CheckAndUpdateObjects();

	FTrailCurrentStatus Status;
	
	CachedHierarchyGuid = InSceneContext.YourNode;
	
	bool bTrackUnchanged = true;
	if (InSceneContext.bCheckForChange || TransformDependencies.IsEmpty())
	{
		bTrackUnchanged = !TrailOrSpaceHasChanged();
	}

	UMovieSceneSection* Section = GetSection();
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	FGuid SequencerBinding = FGuid();
	if (Sequencer && Section)
	{ // TODO: expensive, but for some reason Section stays alive even after it is deleted
		Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrackBinding(*Cast<UMovieSceneTrack>(Section->GetOuter()), SequencerBinding);
	}

	if (!Sequencer || !Section || !SequencerBinding.IsValid())
	{
		Status.CacheState = ETrailCacheState::Dead;
		return Status;
	}
	
	FSequencerTrailHierarchy* SequencerTrailHierarchy = static_cast<FSequencerTrailHierarchy*>(InSceneContext.TrailHierarchy);
	CurrentFramesInfo = SequencerTrailHierarchy->GetCurrentFramesInfo();
	if (!bTrackUnchanged || bForceEvaluateNextTick)
	{
		KeyTool->OnSectionChanged();
		Status.CacheState = ETrailCacheState::Stale;
		bForceEvaluateNextTick = false;

		UpdateNeedsEvaluation(InSceneContext.YourNode, SequencerTrailHierarchy);

	}
	else
	{
		Status.CacheState = ETrailCacheState::UpToDate;

	}
	KeyTool->UpdateViewRange(CurrentFramesInfo->ViewRange);

	return Status;
}

void FMovieSceneTransformTrail::Interp(const FFrameNumber& CurrentFrame, FTransform& OutTransform, FTransform& OutParentTransform)
{
	OutTransform = ArrayOfTransforms->Interp(CurrentFrame, CurrentFramesInfo->TransformIndices, CurrentFramesInfo->CurrentFrames);
	OutParentTransform = ParentArrayOfTransforms->Interp(CurrentFrame, CurrentFramesInfo->TransformIndices, CurrentFramesInfo->CurrentFrames);
}


void FMovieSceneTransformTrail::ClearCachedData()
{
	CachedDrawData.PointsToDraw.SetNum(0);
	CachedDrawData.Color.SetNum(0);
}

void FMovieSceneTransformTrail::GetColor(const FFrameNumber& CurrentTime, FColorState& State)
{
	if (State.GetStyle() == EMotionTrailTrailStyle::HeatMap)
	{
		if (State.bFirstFrame == true)
		{
			State.bFirstFrame = false;
			State.StartFrame = CurrentTime;
			float MinVelocity = FLT_MAX;
			float MaxVelocity = FLT_MIN;
			TArray<float> Velocities;
			Velocities.SetNum(CurrentFramesInfo->TransformIndices.Num());
			int32 TransformIndex = CurrentFramesInfo->TransformIndices[0];
			FVector LastPosition = ArrayOfTransforms->Transforms[TransformIndex].GetLocation();
			float AverageValue = 0.0f;
			for (int32 Index = 1; Index < CurrentFramesInfo->TransformIndices.Num(); ++Index)
			{
				TransformIndex = CurrentFramesInfo->TransformIndices[Index];
				const FVector Position = ArrayOfTransforms->Transforms[TransformIndex].GetLocation();
				const float Velocity = (Position - LastPosition).Length();
				LastPosition = Position;
				if (Velocity > MaxVelocity)
				{
					MaxVelocity = Velocity;
				}
				if (Velocity < MinVelocity)
				{
					MinVelocity = Velocity;
				}
				Velocities[Index - 1] = Velocity;
				AverageValue += Velocity;
			}

			AverageValue /= (float)Velocities.Num();
			if (FMath::IsNearlyZero(AverageValue) == false)
			{
				CachedHeatMap.SetNum(CurrentFramesInfo->TransformIndices.Num());
				const float LowValue = MinVelocity;//AverageValue * 0.5f;
				const float HighValue = MaxVelocity;//AverageValue * 2.0f;
				const float HighAverage = HighValue - AverageValue;
				const float AverageLow = AverageValue - LowValue;
				FColor Color;
				for (int32 Index = 0; Index < Velocities.Num() - 1; ++Index)
				{
					float UpperWeight = FMath::Clamp(Velocities[Index] - AverageValue, 0.0f, HighAverage)/ HighAverage;
					float LowerWeight = FMath::Clamp(AverageValue - Velocities[Index], 0.0f, AverageLow) / AverageLow;
					Color.R = (uint8)(UpperWeight * 255);
					Color.G = (uint8)((1.0 - (UpperWeight + LowerWeight)) * 255);
					Color.B = (uint8)(LowerWeight * 255);
					Color.A = Velocities[Index] >= LowValue ? 255 : 0;
					CachedHeatMap[Index] = Color;
				}
			}
			else //average speed is zero so not moving, no heat map color
			{
				CachedHeatMap.SetNum(0);
			}

		}
		if (CachedHeatMap.Num() > 0)
		{
			const int32 Index = (CurrentTime.Value - State.StartFrame.Value) / State.TicksPerFrame.Value;
			if (Index >= 0 && Index <= CachedHeatMap.Num() - 1)
			{
				State.CalculatedColor = CachedHeatMap[Index];
			}
			else
			{
				State.CalculatedColor = FLinearColor::White;
			}
		}
		else
		{
			State.CalculatedColor = FLinearColor::White;
		}
	}
	else
	{
		FTrail::GetColor(CurrentTime, State);
	}
}

void FMovieSceneTransformTrail::ReadyToDrawTrail(FColorState& ColorState, const FCurrentFramesInfo* InCurrentFramesInfo, bool bIsEvaluating, bool bIsPinned)
{
	ColorState.ReadyForTrail(bIsPinned, GetDrawInfo()->GetStyle());
	bool bCalculateColor = (ColorState.GetStyle() == EMotionTrailTrailStyle::Time || ColorState.GetStyle() == EMotionTrailTrailStyle::Dashed);
	GetParentSpaceTransform();

	if (InCurrentFramesInfo->CurrentFrames.Num() > 0)
	{
		if (InCurrentFramesInfo->bViewRangeIsEvalRange == false || (CachedDrawData.PointsToDraw.Num() != CachedDrawData.Color.Num()) || (CachedDrawData.PointsToDraw.Num() != InCurrentFramesInfo->CurrentFrames.Num())
			|| (CachedDrawData.Color.Num() != InCurrentFramesInfo->CurrentFrames.Num()))
		{
			GetTrajectoryPointsForDisplay(*InCurrentFramesInfo, bIsEvaluating, CachedDrawData.PointsToDraw, CachedDrawData.Frames);
			bCalculateColor = true;
		}
		if (CachedDrawData.PointsToDraw.Num() > 1 && bCalculateColor)
		{
			CachedDrawData.Color.SetNum(CachedDrawData.PointsToDraw.Num());
			for (int32 Idx = 1; Idx < CachedDrawData.PointsToDraw.Num(); Idx++)
			{
				const FFrameNumber CurFrame = CachedDrawData.Frames[Idx - 1];
				GetColor(CurFrame, ColorState);
				CachedDrawData.Color[Idx - 1] = ColorState.CalculatedColor;
			}
		}
	}
}
void FMovieSceneTransformTrail::DrawHUD(const FSceneView* View, FCanvas* Canvas)
{
	KeyTool->DrawHUD(View, Canvas);
}

TOptional<FLinearColor> FMovieSceneTransformTrail::GetOverrideColor() const
{
	TOptional<FLinearColor> Color;
	if (IsTrailSelected())
	{
		if (bIsOffsetMode)
		{
			Color = FLinearColor::Red;
		}
		else
		{
			Color = FLinearColor::Yellow;
		}
	}
	return Color;
}

void FMovieSceneTransformTrail::InternalDrawTrail(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, const FDrawCacheData& InDrawData, bool bHitTesting, bool bTrailIsEvaluating) const
{
	if (bTrailIsEvaluating == false || bIsOffsetMode == true) // if evaluating we now just draw betweeen the keys
	{
		const TOptional<FLinearColor> OverrideColor = GetOverrideColor();
		const float TrailThickness = UMotionTrailToolOptions::GetTrailOptions()->TrailThickness;
		FVector LastPoint = InDrawData.PointsToDraw[0];

		for (int32 Idx = 1; Idx < InDrawData.PointsToDraw.Num(); Idx++)
		{
			const FFrameNumber CurFrame = InDrawData.Frames[Idx - 1];
			if (bHitTesting)
			{
				PDI->SetHitProxy(new HNewMotionTrailProxy(Guid, LastPoint, CurFrame));
			}
			const FVector CurPoint = InDrawData.PointsToDraw[Idx];
			const FLinearColor CurColor = OverrideColor.IsSet() == false ? InDrawData.Color[Idx - 1] : OverrideColor.GetValue();
			PDI->DrawLine(LastPoint, CurPoint, CurColor, SDPG_Foreground, TrailThickness);
			LastPoint = CurPoint;
			if (bHitTesting)
			{
				PDI->SetHitProxy(nullptr);
			}
		}
	}
}

void FMovieSceneTransformTrail::Render(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, bool bTrailIsEvaluating)
{
	if (CachedDrawData.PointsToDraw.Num() > 1)
	{
		const bool bHitTesting = PDI && PDI->IsHitTesting();
		InternalDrawTrail(Guid, View, PDI, CachedDrawData, bHitTesting, bTrailIsEvaluating);
	}
	else if (bIsOffsetMode && bTrailIsEvaluating)
	{
		RenderEvaluating(Guid, View, PDI);
	}
	//render key tool
	KeyTool->Render(Guid, View, PDI, bTrailIsEvaluating);
}

void FMovieSceneTransformTrail::RenderEvaluating(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (PreviousCachedDrawData.PointsToDraw.Num() > 0)
	{
		const bool bHitTesting = false;
		InternalDrawTrail(Guid, View, PDI, PreviousCachedDrawData, bHitTesting, false); //if rendering cached we say not evaluating
	}
}

bool FMovieSceneTransformTrail::HandleAltClick(FEditorViewportClient* InViewportClient, HNewMotionTrailProxy* Proxy, FInputClick Click)
{
	return false;
}

bool FMovieSceneTransformTrail::HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, FInputClick Click)
{
	ExitOffsetMode();

	bool bKeySelected = KeyTool->HandleClick(Guid,InViewportClient, InHitProxy, Click);
	if (bKeySelected)
	{
		if (Click.bShiftIsDown == false || Click.bCtrlIsDown == false)
		{
			bIsSelected = false;
			ExitOffsetMode();
		}
		return true;
	}

	if (HNewMotionTrailProxy* HitProxy = HitProxyCast<HNewMotionTrailProxy>(InHitProxy))
	{
		if (HitProxy->Guid == Guid)
		{

			//	FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), IsInLevelEditor() && !GIsTransacting);
			if (Click.bAltIsDown)
			{
				return HandleAltClick(InViewportClient, HitProxy, Click);
			}
			if (Click.bShiftIsDown)
			{
				bIsSelected = true;
				SelectedPos = HitProxy->Point;
			}
			else if (Click.bCtrlIsDown)
			{
				if (bIsSelected)
				{
					bIsSelected = false;
					ExitOffsetMode();
				}
				else
				{
					bIsSelected = true;
					SelectedPos = HitProxy->Point;
				}
			}
			else
			{
				KeyTool->ClearSelection();
				bIsSelected = true;
				SelectedPos = HitProxy->Point;
			}
			SelectedRotation = FQuat::Identity;
			return true;
		}
	}

	bIsSelected = false;
	ExitOffsetMode();
	return false;
}

bool FMovieSceneTransformTrail::IsAnythingSelected(TArray<FVector>& OutVectorPositions) const
{
	if (bIsSelected)
	{
		const FTransform ParentSpaceTransform = GetParentSpaceTransform();
		TArray<FTrailKeyInfo*> AllKeys;
		KeyTool->GetAllKeys(AllKeys);
		for (const FTrailKeyInfo* KeyInfo : AllKeys)
		{
			if (KeyInfo)
			{
				const FTransform Transform = KeyInfo->Transform * OffsetTransform * ParentSpaceTransform;
				OutVectorPositions.Add(KeyInfo->Transform.GetLocation());
			}
		}
		return true;
	}
	else
	{
		const int32 CurrentSelected = OutVectorPositions.Num();
		KeyTool->GetSelectedKeyPositions(OutVectorPositions);
		if (CurrentSelected != OutVectorPositions.Num())
		{
			return true;
		}
	}
	return false;
}

bool FMovieSceneTransformTrail::IsAnythingSelected(FVector& OutVectorPosition, FQuat& OutRotation) const
{
	if (bIsSelected)
	{
		OutVectorPosition = SelectedPos;
		OutRotation = SelectedRotation;
		return true;
	}
	return KeyTool->IsSelected(OutVectorPosition, OutRotation);
}

void FMovieSceneTransformTrail::TranslateSelectedKeys(bool bRight)
{
	KeyTool->TranslateSelectedKeys(bRight);
}

void FMovieSceneTransformTrail::DeleteSelectedKeys()
{
	KeyTool->DeleteSelectedKeys();
}

void FMovieSceneTransformTrail::SelectNone()
{
	KeyTool->ClearSelection();
	bIsSelected = false;
	ExitOffsetMode();

}

bool FMovieSceneTransformTrail::IsAnythingSelected() const
{
	return (bIsSelected || KeyTool->IsSelected());
}

bool FMovieSceneTransformTrail::IsTrailSelected() const
{
	return bIsSelected;
}
void FMovieSceneTransformTrail::ExitOffsetMode()
{
	bIsOffsetMode = false;
	if (bSetShowWidget.IsSet() && bSetShowWidget.GetValue() == true)
	{
		GLevelEditorModeTools().SetShowWidget(true); //turn widget back on
		bSetShowWidget.Reset();
	}
}
void FMovieSceneTransformTrail::ClearOffsetTransform()
{
	FTrail::ClearOffsetTransform();
	OffsetTransform = FTransform::Identity;
	if (bIsOffsetMode)
	{
		ExitOffsetMode();
		bIsSelected = false;
		CalculateUpdatedOffset();
		ClearCachedData();
	}
}
bool FMovieSceneTransformTrail::GetOffsetMode() const
{
	return bIsOffsetMode;
}

void FMovieSceneTransformTrail::SetOffsetMode()
{
	if (bIsOffsetMode == false)
	{
		SelectNone();
		if (GLevelEditorModeTools().GetShowWidget())
		{
			bSetShowWidget = true;
			GLevelEditorModeTools().SetShowWidget(false);
		}
		bIsOffsetMode = true;
		bIsSelected = true;
		//set selected position aka gizmo location  to be location of the current object
		FFrameTime StartTime = GetSequencer()->GetLocalTime().Time;
		const FTransform Transform = ArrayOfTransforms->Interp(StartTime.GetFrame(), CurrentFramesInfo->TransformIndices, CurrentFramesInfo->CurrentFrames);
		SelectedPos = Transform.GetLocation();
		SelectedRotation = Transform.GetRotation();
		CalculateUpdatedOffset();
	}

}

bool FMovieSceneTransformTrail::IsTracking() const 
{ 
	return bIsTracking;
}

bool FMovieSceneTransformTrail::StartTracking()
{
	bIsTracking = true;
	/* not sure we want this
	//if in offset mode this happened from the tool bar menu,
	// if not we check fo shift is held we go into offset mode
	if (!bIsOffsetMode)
	{
		bIsOffsetMode = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	}
	*/
	return false;
}

void FMovieSceneTransformTrail::CalculateUpdatedOffset()
{
	if (ArrayOfTransforms->Num() == CalculatedArrayOfTransforms->Num())
	{
		ClearCachedData();  
		CacheState = ETrailCacheState::UpToDate;
		for (int32 Index = 0; Index < ArrayOfTransforms->Num(); ++Index)
		{
			FVector InOffsetSpace = CalculatedArrayOfTransforms->Transforms[Index].GetRotation().RotateVector(OffsetTransform.GetLocation());
			InOffsetSpace += CalculatedArrayOfTransforms->Transforms[Index].GetLocation();
			FTransform NewOffset(CalculatedArrayOfTransforms->Transforms[Index].GetRotation().Rotator(),
				InOffsetSpace, CalculatedArrayOfTransforms->Transforms[Index].GetScale3D());
			ArrayOfTransforms->Transforms[Index] = OffsetTransform * CalculatedArrayOfTransforms->Transforms[Index];
		}
		if (ParentSpace.bIsValid && ParentArrayOfTransforms->Num() == ArrayOfTransforms->Num())
		{
			TSet<int32> CalculatedIndices;
			for (int32 Index = 0; Index < ArrayOfTransforms->Num(); ++Index)
			{
				ArrayOfTransforms->Transforms[Index] = ArrayOfTransforms->Transforms[Index].GetRelativeTransform(ParentSpace.ParentSpace.WorldTransforms->Transforms[Index]);
				ParentArrayOfTransforms->Transforms[Index] = ParentArrayOfTransforms->Transforms[Index].GetRelativeTransform(ParentSpace.ParentSpace.WorldTransforms->Transforms[Index]);
			}
		}
		KeyTool->DirtyKeyTransforms();
		KeyTool->UpdateKeys();
	}
}
bool FMovieSceneTransformTrail::ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset)
{
	if (bApplyToOffset || bIsOffsetMode)
	{
		if (IsTrailSelected())
		{
			FQuat Rotation(Rot);
			FVector UnRotatedVector = Rotation.UnrotateVector(Pos);
			OffsetTransform.SetLocation(OffsetTransform.GetLocation() + UnRotatedVector);
			if (bIsOffsetMode == false)
			{
				int32 Index = UMotionTrailToolOptions::GetTrailOptions()->GetIndexFromGuid(CachedHierarchyGuid);
				if (Index != INDEX_NONE)
				{
					UMotionTrailToolOptions::GetTrailOptions()->SetHasOffset(Index, true);
				}
			}
			SelectedPos += Pos;
			CalculateUpdatedOffset();
			return true;
		}
	}
	return false;
}

bool FMovieSceneTransformTrail::EndTracking()
{
	//bIsTracking = false; We don't do this since we need to stop tracking once evaluating is done
	return false;
}

TArray<FFrameNumber> FMovieSceneTransformTrail::GetKeyTimes() const
{
	TArray<FTrailKeyInfo*> AllKeys;
	KeyTool->GetAllKeys(AllKeys);

	TArray<FFrameNumber> Frames;
	for (FTrailKeyInfo* Info : AllKeys)
	{
		if (Info)
		{
			Frames.Add(Info->FrameNumber);
		}
	}
	return Frames;
}

TArray<FFrameNumber> FMovieSceneTransformTrail::GetSelectedKeyTimes() const
{
	return KeyTool->SelectedKeyTimes();
}

void FMovieSceneTransformTrail::UpdateNeedsEvaluation(const FGuid& InTrailGuid, FSequencerTrailHierarchy* SequencerHierarchy)
{
	if (ParentSpace.bIsValid)
	{
		FActorForWorldTransforms Actors;
		Actors.Actor = ParentSpace.ParentSpace.Actor.Actor;
		Actors.Component = ParentSpace.ParentSpace.Actor.Component;
		SequencerHierarchy->EvaluateActor(InTrailGuid, Actors, ParentSpace.ParentSpace.WorldTransforms, ParentSpace.ParentSpace.ParentTransforms);
	}
}

void FMovieSceneTransformTrail::FMovieSceneTransformTrailState::SaveFromTrail(const FMovieSceneTransformTrail* InTrail)
{
	if (InTrail)
	{
		BindingID = InTrail->BindingID;
		WeakSequencer = InTrail->WeakSequencer;
		MainTrack = InTrail->MainTrack;
		OffsetTransform = InTrail->OffsetTransform;
		ParentSpace = InTrail->ParentSpace;
		if (InTrail->GetDrawInfo())
		{
			Color = InTrail->GetDrawInfo()->GetColor();
			PinnedStyle = InTrail->GetDrawInfo()->GetStyle();
		}
	}
}

void FMovieSceneTransformTrail::FMovieSceneTransformTrailState::SetToTrail(FMovieSceneTransformTrail* InTrail) const
{
	if (InTrail)
	{
		if (InTrail->GetDrawInfo())
		{
			InTrail->GetDrawInfo()->SetColor(Color);
			InTrail->GetDrawInfo()->SetStyle(PinnedStyle);
		}
		InTrail->OffsetTransform = OffsetTransform;
		InTrail->ParentSpace = ParentSpace;
	}
}

/****************************************************************************************************************
*
*	Movie Scene Transform Component Trail
*
*****************************************************************************************************************/

void FMovieSceneComponentTransformTrail::UpdateNeedsEvaluation(const FGuid& InTrailGuid, FSequencerTrailHierarchy* SequencerHierarchy)
{
	if (Component.IsValid())
	{
		FMovieSceneTransformTrail::UpdateNeedsEvaluation(InTrailGuid, SequencerHierarchy);
		FActorForWorldTransforms Actors;
		Actors.Actor = Component->GetOwner();
		Actors.Component = Component;
		SequencerHierarchy->EvaluateActor(InTrailGuid, Actors, CalculatedArrayOfTransforms, ParentArrayOfTransforms);
		ArrayOfTransforms->Transforms.SetNum(CalculatedArrayOfTransforms->Transforms.Num());

	}
}

bool FMovieSceneComponentTransformTrail::HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap)
{
	bool bSomethingChanged = FMovieSceneTransformTrail::HandleObjectsChanged(ReplacementMap);

	if (UObject* const* NewObject = ReplacementMap.Find(Component.Get()))
	{
		Component = Cast<USceneComponent>(*NewObject);
		bSomethingChanged = bSomethingChanged || true;
	}
		
	return bSomethingChanged;
}

FText FMovieSceneComponentTransformTrail::GetName() const
{
	if (Component.IsValid())
	{
		FString ActorLabel = Component->GetOwner()->GetActorLabel(false);
		return FText::FromString(ActorLabel);
	}
	return FText();
}

bool FMovieSceneComponentTransformTrail::StartTracking()
{
	FMovieSceneTransformTrail::StartTracking();

	UMovieSceneSection* Section = GetSection();
	if (!Section)
	{
		return false;
	}

	if (IsAnythingSelected())
	{
		bStartTracking = true;
		KeyTool->StartDragging();
		return true;
	}
	return false;
}

bool FMovieSceneComponentTransformTrail::ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset)
{
	if (bApplyToOffset || bIsOffsetMode)
	{
		return FMovieSceneTransformTrail::ApplyDelta(Pos, Rot, WidgetLocation, bApplyToOffset);
	}
	if (bStartTracking)
	{
		bStartTracking = false;
		UMovieSceneSection* Section = GetSection();
		if (!Section)
		{
			return false;
		}
	}
	if (Component.IsValid() == false)
	{
		return false;
	}
	UMovieSceneSection* Section = GetSection();
	if (!Section)
	{
		return false;
	}
	UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section);
	if (!Section)
	{
		return false;
	}

	if (Pos.IsNearlyZero() && Rot.IsNearlyZero())
	{
		return false;
	}

	if (IsAnythingSelected())
	{
		TArray<FTransform> Transforms;
		TArray<FTransform> ParentTransforms;
		TArray<FFrameNumber> Times;
		FTransform InverseOffsetTransform = OffsetTransform.Inverse();
		if (IsTrailSelected())
		{
			TArray<FTrailKeyInfo*> Keys;
			KeyTool->GetAllKeys(Keys);
			for (FTrailKeyInfo* KeyInfo : Keys)
			{
				FTransform NewTransform(KeyInfo->Transform);
				NewTransform.SetLocation(NewTransform.GetLocation() + Pos);
				KeyInfo->Transform = NewTransform;
				NewTransform = OffsetTransform.GetRelativeTransformReverse(NewTransform);
				Transforms.Add(NewTransform);
				ParentTransforms.Add(KeyInfo->ParentTransform);
				Times.Add(KeyInfo->FrameNumber);
			}
			SelectedPos += Pos;
		}
		else
		{
			for (FTrailKeyInfo* KeyInfo : KeyTool->CachedSelection)
			{
				FTransform NewTransform(KeyInfo->Transform);
				NewTransform.SetLocation(NewTransform.GetLocation() + Pos);
				KeyInfo->Transform = NewTransform;
				NewTransform = OffsetTransform.GetRelativeTransformReverse(NewTransform);

				Transforms.Add(NewTransform);
				ParentTransforms.Add(KeyInfo->ParentTransform);
				Times.Add(KeyInfo->FrameNumber);
			}
		}
		UE::AIE::FSetTransformHelpers::SetActorTransform(GetSequencer().Get(), Component.Get(), TransformSection, Times, Transforms, ParentTransforms);
		KeyTool->UpdateSelectedKeysTransform();
		return true;
	}
	return false;
}

bool FMovieSceneComponentTransformTrail::EndTracking()
{
	bStartTracking = false;
	FMovieSceneTransformTrail::EndTracking();
	UMovieSceneSection* Section = GetSection();
	if (!Section)
	{
		return false;
	}

	if (IsAnythingSelected())
	{
		//need to broadcast this so we can update other systems(including our own).
		AActor* Actor = Component.Get()->GetTypedOuter<AActor>();
		GEngine->BroadcastOnActorMoved(Actor);
		return true;
	}
	return false;
}

bool FMovieSceneComponentTransformTrail::HandleAltClick( FEditorViewportClient* InViewportClient, HNewMotionTrailProxy* Proxy,FInputClick Click)
{
	if (Component.IsValid() == false)
	{
		return false;
	}
	UMovieSceneSection* Section = GetSection();
	if (!Section)
	{
		return false;
	}
	UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section);
	if (!Section)
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("InsertKey", "Insert Key"));

	TArray<FFrameNumber> Times;
	TArray<FTransform> Transforms;
	TArray<FTransform> ParentTransforms;
	FFrameRate TickResolution = GetSequencer()->GetFocusedTickResolution();
	FFrameRate DisplayResolution = GetSequencer()->GetFocusedDisplayRate();
	//Snap to Frame. Assuming we want keys on frames.
	FFrameTime GlobalTime = FFrameTime(Proxy->CurrentFrame,0.0);
	FFrameTime DisplayTime = FFrameRate::TransformTime(GlobalTime, DisplayResolution, TickResolution);
	DisplayTime = DisplayTime.RoundToFrame();
	FFrameTime EvalTime = FFrameRate::TransformTime(DisplayTime, TickResolution, DisplayResolution);
	Times.Add(EvalTime.RoundToFrame());
	const FTransform Transform = ArrayOfTransforms->Interp(Proxy->CurrentFrame,CurrentFramesInfo->TransformIndices, CurrentFramesInfo->CurrentFrames);
	const FTransform ParentTransform = ParentArrayOfTransforms->Interp(Proxy->CurrentFrame, CurrentFramesInfo->TransformIndices, CurrentFramesInfo->CurrentFrames);
	Transforms.Add(Transform);
	ParentTransforms.Add(ParentTransform);
	UE::AIE::FSetTransformHelpers::SetActorTransform(GetSequencer().Get(), Component.Get(), TransformSection, Times, Transforms, ParentTransforms);
	KeyTool->UpdateSelectedKeysTransform();
	return true;
}

TUniquePtr<FMovieSceneTransformTrail::FMotionTrailState> FMovieSceneComponentTransformTrail::GetMotionTrailState() const
{
	TUniquePtr<FMovieSceneComponentTransformTrail::FMovieSceneComponentTransformTrailState> State = MakeUnique<FMovieSceneComponentTransformTrail::FMovieSceneComponentTransformTrailState>();
	State->SaveFromTrail(this);
	State->Component = Component;
	return MoveTemp(State);
}

void FMovieSceneComponentTransformTrail::FMovieSceneComponentTransformTrailState::RestoreTrail(FTrailHierarchy* InTrailHierarchy)
{
	FSequencerTrailHierarchy* TrailHierarchy = static_cast<FSequencerTrailHierarchy*>(InTrailHierarchy);
	USceneComponent* SceneComp = Component.Pin().Get();
	if (!SceneComp)
	{
		return;
	}
	UMovieScene3DTransformTrack* Track = Cast< UMovieScene3DTransformTrack>(MainTrack.Get());
	if (!Track)
	{
		return;
	}
	FGuid NewGuid = TrailHierarchy->AddComponentToHierarchy(BindingID, SceneComp, Track);
	TrailHierarchy->PinTrail(NewGuid);
	if (const TUniquePtr<FTrail>* Trail = TrailHierarchy->GetAllTrails().Find(NewGuid))
	{
		if (FMovieSceneTransformTrail* TransformTrail = static_cast<FMovieSceneTransformTrail*>(Trail->Get()))
		{
			SetToTrail(TransformTrail);
		}
	}

}

/****************************************************************************************************************
*
*Movie Scene Socket Component Trail
*
*****************************************************************************************************************/

void FMovieSceneSocketTransformTrail::UpdateNeedsEvaluation(const FGuid& InTrailGuid, FSequencerTrailHierarchy * SequencerHierarchy)
{
	if (Component.IsValid())
	{
		FMovieSceneTransformTrail::UpdateNeedsEvaluation(InTrailGuid, SequencerHierarchy);
		FActorForWorldTransforms Actors;
		Actors.Actor = Component->GetOwner();
		Actors.Component = Component;
		Actors.SocketName = SocketName;
		SequencerHierarchy->EvaluateActor(InTrailGuid, Actors, CalculatedArrayOfTransforms, ParentArrayOfTransforms);
		ArrayOfTransforms->Transforms.SetNum(CalculatedArrayOfTransforms->Transforms.Num());
	}
}

bool FMovieSceneSocketTransformTrail::HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap)
{
	bool bSomethingChanged = FMovieSceneTransformTrail::HandleObjectsChanged(ReplacementMap);

	if (UObject* const* NewObject = ReplacementMap.Find(Component.Get()))
	{
		Component = Cast<USceneComponent>(*NewObject);
		bSomethingChanged = bSomethingChanged || true;
	}

	return bSomethingChanged;
}


FText FMovieSceneSocketTransformTrail::GetName() const
{
	if (Component.IsValid())
	{
		FString ActorLabel = Component->GetOwner()->GetActorLabel(false);
		FString SocketNameString = SocketName.ToString();

		SocketNameString = FString::Printf(TEXT("%s:%s"), *ActorLabel,*SocketNameString);
		return FText::FromString(SocketNameString);
	}
	return FText();
}

bool FMovieSceneSocketTransformTrail::StartTracking()
{

	FMovieSceneTransformTrail::StartTracking();
	if (IsAnythingSelected())
	{
		return true;
	}
	return false;
}


bool FMovieSceneSocketTransformTrail::ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset)
{
	if (bApplyToOffset || bIsOffsetMode)
	{
		return FMovieSceneTransformTrail::ApplyDelta(Pos, Rot, WidgetLocation, bApplyToOffset);
	}
	return false;
}

bool FMovieSceneSocketTransformTrail::EndTracking()
{
	FMovieSceneTransformTrail::EndTracking();

	if (IsAnythingSelected())
	{
		//need to broadcast this so we can update other systems(including our own).
		AActor* Actor = Component.Get()->GetTypedOuter<AActor>();
		GEngine->BroadcastOnActorMoved(Actor);
		return true;
	}
	return false;
}


void FMovieSceneSocketTransformTrail::DrawHUD(const FSceneView* View, FCanvas* Canvas)
{
	//todo for ticks
}

bool FMovieSceneSocketTransformTrail::HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, FInputClick Click)
{
	FMovieSceneTransformTrail::HandleClick(Guid, InViewportClient, InHitProxy, Click);
	if (HNewMotionTrailProxy* HitProxy = HitProxyCast<HNewMotionTrailProxy>(InHitProxy))
	{
		if (HitProxy->Guid == Guid)
		{

			if (Click.bShiftIsDown)
			{
				bIsSelected = true;
				SelectedPos = HitProxy->Point;
			}
			else if (Click.bCtrlIsDown)
			{
				if (bIsSelected)
				{
					bIsSelected = false;
					ExitOffsetMode();
				}
				else
				{
					bIsSelected = true;
					SelectedPos = HitProxy->Point;
				}
			}
			else
			{
				KeyTool->ClearSelection();
				bIsSelected = true;
				SelectedPos = HitProxy->Point;
			}
			return true;
		}
	}
	bIsSelected = false;
	ExitOffsetMode();
	return false;
}


bool FMovieSceneSocketTransformTrail::IsAnythingSelected(FVector& OutVectorPosition, FQuat& OutRotation) const
{
	if (bIsSelected)
	{
		OutVectorPosition = SelectedPos;
		OutRotation = FQuat::Identity;
		return true;
	}
	return false;
}

bool FMovieSceneSocketTransformTrail::IsAnythingSelected() const
{
	return (bIsSelected );
}

TUniquePtr<FMovieSceneTransformTrail::FMotionTrailState> FMovieSceneSocketTransformTrail::GetMotionTrailState() const
{
	TUniquePtr<FMovieSceneSocketTransformTrail::FMovieSceneSocketTransformTrailState> State = MakeUnique<FMovieSceneSocketTransformTrail::FMovieSceneSocketTransformTrailState>();
	State->SaveFromTrail(this);
	State->Component = Component;
	State->SocketName = SocketName;
	State->Color = GetDrawInfo()->GetColor();
	State->PinnedStyle = GetDrawInfo()->GetStyle();
	State->Color = GetDrawInfo()->GetColor();
	State->PinnedStyle = GetDrawInfo()->GetStyle();
	return MoveTemp(State);
}

void FMovieSceneSocketTransformTrail::FMovieSceneSocketTransformTrailState::RestoreTrail(FTrailHierarchy* InTrailHierarchy)
{
	FSequencerTrailHierarchy* TrailHierarchy = static_cast<FSequencerTrailHierarchy*>(InTrailHierarchy);
	USceneComponent* SceneComp = Component.Pin().Get();
	if (!SceneComp)
	{
		return;
	}
	FGuid NewGuid = TrailHierarchy->PinComponent(SceneComp, SocketName);
	TrailHierarchy->PinTrail(NewGuid);
	if (const TUniquePtr<FTrail>* Trail = TrailHierarchy->GetAllTrails().Find(NewGuid))
	{
		if (FMovieSceneTransformTrail* TransformTrail = static_cast<FMovieSceneTransformTrail*>(Trail->Get()))
		{
			SetToTrail(TransformTrail);
		}
	}
}

/****************************************************************************************************************
*
*   Control Rig Transform Trai
*
*****************************************************************************************************************/

FMovieSceneControlRigTransformTrail::FMovieSceneControlRigTransformTrail(const FGuid& InBindingGuid, USceneComponent* SceneComponent,  const bool bInIsVisible, TWeakObjectPtr<UMovieSceneTrack> InWeakTrack, 
	TSharedPtr<ISequencer> InSequencer,  const FName& InControlName, TSharedPtr<UE::AIE::FArrayOfTransforms>& InParentArrayOfTransforms)
: FMovieSceneTransformTrail(InBindingGuid, SceneComponent, bInIsVisible, InWeakTrack, InSequencer)
, ControlName(InControlName)
{
	ParentArrayOfTransforms = InParentArrayOfTransforms;
}

void FMovieSceneControlRigTransformTrail::UpdateNeedsEvaluation(const FGuid& InTrailGuid, FSequencerTrailHierarchy* SequencerHierarchy)
{
	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(GetSection());
	if (!Section)
	{
		return;
	}
	UControlRig* ControlRig = Section->GetControlRig();
	if (!ControlRig)
	{
		return;
	}
	FMovieSceneTransformTrail::UpdateNeedsEvaluation(InTrailGuid, SequencerHierarchy);

	SequencerHierarchy->EvaluateControlRig(InTrailGuid, ControlRig, ControlName, CalculatedArrayOfTransforms);
	ArrayOfTransforms->Transforms.SetNum(CalculatedArrayOfTransforms->Transforms.Num());
}

FText FMovieSceneControlRigTransformTrail::GetName() const
{
	return FText::FromString(ControlName.ToString());
}



bool FMovieSceneControlRigTransformTrail::StartTracking()
{
	FMovieSceneTransformTrail::StartTracking();

	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(GetSection());
	if (!Section)
	{
		return false;
	}
	UControlRig* ControlRig = Section->GetControlRig();
	if (!ControlRig)
	{
		return false ;
	}
	if(IsAnythingSelected())
	{ 
		if ( bIsOffsetMode == false)
		{
			bStartTracking = true;
		}
		KeyTool->StartDragging();
		return true;
	}
	return false;
}

int32 FMovieSceneControlRigTransformTrail::GetChannelOffset() const
{
	UMovieSceneControlRigParameterSection* CRParamSection = Cast<UMovieSceneControlRigParameterSection>(GetSection());
	if (CRParamSection)
	{
		FChannelMapInfo* pChannelIndex = CRParamSection->ControlChannelMap.Find(ControlName);
		return pChannelIndex ? pChannelIndex->ChannelIndex : INDEX_NONE;
	}
	return INDEX_NONE;
}

bool FMovieSceneControlRigTransformTrail::ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset)
{
	if (bApplyToOffset || bIsOffsetMode)
	{
		return FMovieSceneTransformTrail::ApplyDelta(Pos, Rot, WidgetLocation, bApplyToOffset);
	}

	if (bStartTracking)
	{
		bStartTracking = false;
		UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(GetSection());
		if (!Section)
		{
			return false;
		}
		UControlRig* ControlRig = Section->GetControlRig();
		if (!ControlRig)
		{
			return false;
		}
		Section->Modify();
		ControlRig->Modify();
	}
	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(GetSection());
	if (!Section)
	{
		return false;
	}
	UControlRig* ControlRig = Section->GetControlRig();
	if (!ControlRig)
	{
		return false;
	}

	if (Pos.IsNearlyZero() && Rot.IsNearlyZero())
	{
		return false;
	}

	if (IsAnythingSelected())
	{
		FRigControlElement* ControlElement = ControlRig->GetHierarchy()->Find<FRigControlElement>((FRigElementKey(ControlName, ERigElementType::Control)));
		if (ControlElement == nullptr)
		{
			return false;
		}

		FFrameRate TickResolution = GetSequencer()->GetFocusedTickResolution();
		FRigControlModifiedContext Context;
		Context.SetKey = EControlRigSetKey::Always;
		IMovieScenePlayer* Player = GetSequencer().Get();

		FMovieSceneSequenceTransform RootToLocalTransform = GetSequencer()->GetFocusedMovieSceneSequenceTransform();

		auto EvalControlRig = [&Context,&RootToLocalTransform, Pos, TickResolution, ControlRig,Player, ControlElement, this](FTrailKeyInfo* KeyInfo)
		{
			if (KeyInfo)
			{
				//the keys are in warped time which is what we want to set the control rig values with, but we want to evaluate in unwarped time
				FFrameTime GlobalTime(KeyInfo->FrameNumber);
				GlobalTime = RootToLocalTransform.TransformTime(GlobalTime);
				Context.LocalTime = TickResolution.AsSeconds(GlobalTime);
				Context.KeyMask = (uint32)EControlRigContextChannelToKey::Translation;
				GlobalTime = RootToLocalTransform.Inverse().TryTransformTime(GlobalTime).Get(GlobalTime);//back to unwarped
				FMovieSceneContext MovieSceneContext = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);

				FTransform NewTransform(KeyInfo->Transform);
				NewTransform.SetLocation(NewTransform.GetLocation() + Pos);
				KeyInfo->Transform = NewTransform;

				NewTransform = OffsetTransform.GetRelativeTransformReverse(NewTransform);

				NewTransform = NewTransform.GetRelativeTransform(KeyInfo->ParentTransform);
				GetSequencer()->GetEvaluationTemplate().EvaluateSynchronousBlocking(MovieSceneContext);
				ControlRig->Evaluate_AnyThread();
				constexpr ERigTransformType::Type TransformType = ERigTransformType::CurrentGlobal;

				FRigControlValue Value = ControlRig->GetControlValueFromGlobalTransform(ControlElement->GetKey().Name, NewTransform,TransformType);
				NewTransform = Value.GetAsTransform(ControlElement->Settings.ControlType,ControlElement->Settings.PrimaryAxis);
				FEulerTransform EulerTransform(NewTransform);
				UE::AIE::FSetTransformHelpers::SetControlTransform(EulerTransform,ControlRig, ControlElement, Context);		
			}
		};
		FControlRigEditMode::FTurnOffPosePoseUpdate  TurnOff; //stop flashing

		if (IsTrailSelected())
		{
			TArray<FTrailKeyInfo*> Keys;
			KeyTool->GetAllKeys(Keys);
			for (FTrailKeyInfo* KeyInfo : Keys)
			{
				EvalControlRig(KeyInfo);
			}
			SelectedPos += Pos;
		}
		else
		{
			for (FTrailKeyInfo* KeyInfo : KeyTool->CachedSelection)
			{
				EvalControlRig(KeyInfo);
			}
		}
		KeyTool->UpdateSelectedKeysTransform();
		GetSequencer()->ForceEvaluate();
		ControlRig->Evaluate_AnyThread();
		if (ControlRig->GetObjectBinding())
		{
			ControlRig->EvaluateSkeletalMeshComponent(0.0);
		}
		return true;
	}
	return false;
}

bool FMovieSceneControlRigTransformTrail::EndTracking()
{
	FMovieSceneTransformTrail::EndTracking();
	bStartTracking = false;

	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(GetSection());
	if (!Section)
	{
		return false;
	}
	UControlRig* ControlRig = Section->GetControlRig();
	if (!ControlRig)
	{
		return false;
	}

	if (IsAnythingSelected())
	{
		IMovieScenePlayer* Player = GetSequencer().Get();
		FFrameTime StartTime = GetSequencer()->GetLocalTime().Time;
		FFrameRate TickResolution = GetSequencer()->GetFocusedTickResolution();
		FMovieSceneInverseSequenceTransform LocalToRootTransform = GetSequencer()->GetFocusedMovieSceneSequenceTransform().Inverse();
		StartTime = LocalToRootTransform.TryTransformTime(StartTime).Get(StartTime); //player evals in root time so need to go back to it.

		FMovieSceneContext MovieSceneContext = FMovieSceneContext(FMovieSceneEvaluationRange(StartTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);
		
		Player->GetEvaluationTemplate().EvaluateSynchronousBlocking(MovieSceneContext);
		ControlRig->Evaluate_AnyThread();
		return true;
	}
	return false;
}

bool FMovieSceneControlRigTransformTrail::HandleAltClick(FEditorViewportClient* InViewportClient, HNewMotionTrailProxy* Proxy, FInputClick Click)
{
	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(GetSection());
	if (!Section)
	{
		return false;
	}
	UControlRig* ControlRig = Section->GetControlRig();
	if (!ControlRig)
	{
		return false;
	}
	const FScopedTransaction Transaction(LOCTEXT("InsertKey", "Insert Key"));
	Section->Modify();
	ControlRig->Modify();
	FFrameRate TickResolution = GetSequencer()->GetFocusedTickResolution();
	FFrameRate DisplayResolution = GetSequencer()->GetFocusedDisplayRate();

	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	IMovieScenePlayer* Player = GetSequencer().Get();

	//Snap to Frame. Assuming we want keys on frames.
	FFrameTime GlobalTime = FFrameTime(Proxy->CurrentFrame,0.0);
	FFrameTime DisplayTime = FFrameRate::TransformTime(GlobalTime, DisplayResolution, TickResolution);
	DisplayTime = DisplayTime.RoundToFrame();
	GlobalTime = FFrameRate::TransformTime(DisplayTime, TickResolution, DisplayResolution);

	FMovieSceneInverseSequenceTransform LocalToRootTransform = GetSequencer()->GetFocusedMovieSceneSequenceTransform().Inverse();
	GlobalTime = LocalToRootTransform.TryTransformTime(GlobalTime).Get(GlobalTime); //player evals in root time so need to go back to it.

	Context.LocalTime = TickResolution.AsSeconds(GlobalTime);
	Context.KeyMask = (uint32)EControlRigContextChannelToKey::Translation;
	FMovieSceneContext MovieSceneContext = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);
	GetSequencer()->GetEvaluationTemplate().EvaluateSynchronousBlocking(MovieSceneContext);
	ControlRig->Evaluate_AnyThread();
	//todo handle constraints
	FTransform NewTransform(ControlRig->GetControlGlobalTransform(ControlName));

	//test constraints
	ControlRig->SetControlGlobalTransform(ControlName, NewTransform, true, Context, false /*undo*/, false /*bPrintPython*/, true/* bFixEulerFlips*/);

	//eval back at current time
	FFrameTime StartTime = GetSequencer()->GetGlobalTime().Time;

	MovieSceneContext = FMovieSceneContext(FMovieSceneEvaluationRange(StartTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);
	Player->GetEvaluationTemplate().EvaluateSynchronousBlocking(MovieSceneContext);
	ControlRig->Evaluate_AnyThread();

	//create new keys
	KeyTool->BuildKeys();
	FTrailKeyInfo* KeyInfo = KeyTool->FindKey(GlobalTime.RoundToFrame());
	if (KeyInfo)
	{
		if (Click.bShiftIsDown == false && Click.bCtrlIsDown == false)
		{
			KeyTool->ClearSelection();;
		}
		KeyTool->SelectKeyInfo(KeyInfo);
	}
	return true;
}

TUniquePtr<FMovieSceneTransformTrail::FMotionTrailState> FMovieSceneControlRigTransformTrail::GetMotionTrailState() const
{
	TUniquePtr<FMovieSceneControlRigTransformTrail::FMovieSceneControlRigTransformTrailState> State = MakeUnique<FMovieSceneControlRigTransformTrail::FMovieSceneControlRigTransformTrailState>();
	State->SaveFromTrail(this);
	State->Owner = Cast<USkeletalMeshComponent>(Owner);
	State->ControlName = ControlName;
	return MoveTemp(State);
}

void FMovieSceneControlRigTransformTrail::FMovieSceneControlRigTransformTrailState::RestoreTrail(FTrailHierarchy* InTrailHierarchy)
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		FSequencerTrailHierarchy* TrailHierarchy = static_cast<FSequencerTrailHierarchy*>(InTrailHierarchy);
		USkeletalMeshComponent* SkelMeshComp = Owner.Pin().Get();
		if (!SkelMeshComp)
		{
			return;
		}
		UMovieSceneControlRigParameterTrack* CRTrack = Cast< UMovieSceneControlRigParameterTrack>(MainTrack.Get());
		if (!CRTrack || !CRTrack->GetControlRig())
		{
			return;
		}
		FGuid NewGuid = TrailHierarchy->AddControlRigTrail(SkelMeshComp, CRTrack->GetControlRig(), CRTrack, ControlName);
		TrailHierarchy->PinTrail(NewGuid);
		if (const TUniquePtr<FTrail>* Trail = TrailHierarchy->GetAllTrails().Find(NewGuid))
		{
			if (FMovieSceneTransformTrail* TransformTrail = static_cast<FMovieSceneTransformTrail*>(Trail->Get()))
			{
				SetToTrail(TransformTrail);
			}
		}
	}
}


} // namespace MovieScene
} // namespace UE

#undef LOCTEXT_NAMESPACE
