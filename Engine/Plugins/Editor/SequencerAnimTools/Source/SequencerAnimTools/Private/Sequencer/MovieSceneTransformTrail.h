// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Trail.h"
#include "MotionTrailMovieSceneKey.h"
#include "MovieSceneTrack.h"
#include "Tools/EvaluateSequencerTools.h"
#include "Misc/Optional.h"

class IMovieScenePlayer;

class ISequencer;
class UMovieScene3DTransformTrack;
class UMovieScene3DTransformSection;
class UMovieSceneTrack;
class USceneComponent;

namespace UE
{
namespace SequencerAnimTools
{

class FSequencerTrailHierarchy;

//caching of tracks ans sections to handle signature changes.
struct FCachedGuidsPerSignedObject
{
	TWeakObjectPtr<const UMovieSceneSignedObject> SignedObject;
	FGuid CachedGuid;
};

struct FCachedTrackAndSections
{
	FCachedGuidsPerSignedObject Track;
	TMap<TWeakObjectPtr<const UMovieSceneSection>, FCachedGuidsPerSignedObject> Sections;
};

//space that a trail may be in, includes Actor/ optional component and it's sequencer tracks/sections
struct FOptionalParentSpace
{
	FOptionalParentSpace() : bIsValid(false) { ParentSpace.Actor.Actor = nullptr; ParentSpace.Actor.Component = nullptr; };

	UE::AIE::FActorAndWorldTransforms ParentSpace; //this contains the actor/component
	
	bool bIsValid;
	FName ComponentName;
	FGuid SpaceBindingID;
	UE::AIE::FSequencerTransformDependencies SpaceTransformDependencies;

	//set space and make it valid
	void SetSpace(TSharedPtr<ISequencer>& InSequencerPtr, AActor* InActor, const FName& InComponentName);
	void ClearSpace();
	
	static USceneComponent* GetComponentFromName(const AActor* InActor, const FName& InComponentName);
};

struct FDrawCacheData
{
	TArray<FVector> PointsToDraw;
	TArray<FFrameNumber> Frames;
	TArray<FLinearColor> Color;
};
class FMovieSceneTransformTrail : public FTrail
{
public:

	FMovieSceneTransformTrail(const FGuid& InBindingID, USceneComponent* SceneComponent, const bool bInIsVisible, TWeakObjectPtr<UMovieSceneTrack> InWeakTrack, TSharedPtr<ISequencer> InSequencer);
	~FMovieSceneTransformTrail();


	// Begin FTrail interface
	virtual FTrailCurrentStatus UpdateTrail(const FNewSceneContext& NewSceneContext) override;
	virtual void Interp(const FFrameNumber& Time, FTransform& OutTransform, FTransform& OutParentTransform) override;
	virtual void UpdateFinished(const TRange<FFrameNumber>& UpdatedRange, const TArray<int32>& IndicesToCalcluate, bool bDoneCalculating) override;
	virtual void AddImportantTimes(TSet<FFrameNumber>& InOutImportantTimes) override;
	virtual FTransform GetOffsetTransform() const override { return OffsetTransform; }
	virtual void ClearOffsetTransform() override;
	virtual void SetOffsetMode() override;
	virtual bool GetOffsetMode() const override;

	virtual FTransform GetParentSpaceTransform() const override;
	virtual void SetSpace(AActor* InActor, const FName& InComponentName) override;
	virtual void ClearSpace() override { ParentSpace.ClearSpace(); 	ClearCachedData();}
	virtual void ClearCachedData() override;
	virtual void ReadyToDrawTrail(FColorState& ColorState, const FCurrentFramesInfo* InCurrentFramesInfo, bool bIsEvaluating, bool bIsPinned) override;
	virtual void ActorChanged(AActor* InActor) override;

	virtual void Render(const FGuid& Guid, const FSceneView* View,  FPrimitiveDrawInterface* PDI, bool bTrailIsEvaluating) override;
	virtual void RenderEvaluating(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(const FSceneView* View, FCanvas* Canvas) override;

	virtual bool HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, FInputClick Click) override;
	virtual bool IsAnythingSelected() const override;
	virtual bool IsAnythingSelected(FVector& OutVectorPosition, FQuat& OutRotation)const override;
	virtual bool IsAnythingSelected(TArray<FVector>& OutVectorPositions)const override;
	virtual bool IsTrailSelected() const override;
	virtual bool StartTracking() override;
	virtual bool ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset) override;
	virtual bool EndTracking() override;
	virtual bool IsTracking() const override;

	virtual void HasStartedEvaluating() override;
	virtual void TranslateSelectedKeys(bool bRight) override;
	virtual void DeleteSelectedKeys() override;
	virtual void SelectNone() override;
	virtual int32 GetChannelOffset()const { return 0; }
	virtual TArray<FFrameNumber> GetKeyTimes() const override;
	virtual TArray<FFrameNumber> GetSelectedKeyTimes() const override;
	virtual void ForceEvaluateNextTick() override;
	virtual void GetColor(const FFrameNumber& CurrentTime, FColorState& InOutColorState) override;
	virtual bool HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap) override;

	// End FTrail interface
	
	// New Virtuals
	virtual void UpdateNeedsEvaluation(const FGuid& InTrailGuid, FSequencerTrailHierarchy* SequencerHierarchy);

	TSharedPtr<ISequencer> GetSequencer() const { return WeakSequencer.Pin(); }
	FGuid GetCachedHierarchyGuid() const { return CachedHierarchyGuid; }
	UMovieSceneSection* GetSection() const;

	void SetOffsetTransform(const FTransform& InOffsetTransform) { OffsetTransform = InOffsetTransform; ClearCachedData(); }
	const UE::AIE::FSequencerTransformDependencies& GetTransformDependencies() { return TransformDependencies; }

public:
	TWeakPtr<ISequencer> WeakSequencer;

protected:
	virtual bool HandleAltClick(FEditorViewportClient* InViewportClient, HNewMotionTrailProxy* Proxy, FInputClick Click);

	//internal draw trail
	void InternalDrawTrail(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, const FDrawCacheData& InDrawData, bool bHitTesting, bool bTrailIsEvaluating) const;

	//if selected or selected and offset mode we draw different colors
	TOptional<FLinearColor> GetOverrideColor() const;

protected:
	//do to spawnables, etc.. objects that we have may have been deleted or recreated, need to update them if they have
	virtual void CheckAndUpdateObjects();
	static TArray<UObject*> GetBoundObjects(TSharedPtr<ISequencer>& InSequencerPtr, const FGuid& InGuid);

	void CalculateUpdatedOffset();
protected:
	//some interactive states/modes
	bool bIsSelected;
	bool bIsOffsetMode;
	FVector SelectedPos;
	FQuat SelectedRotation;
	bool bIsTracking; 
	TOptional<bool> bSetShowWidget;

	//main tool for adjusting keys
	TUniquePtr<FMotionTraiMovieScenelKeyTool> KeyTool;

	//previous parent space
	mutable FTransform PreviousParentSpaceTM = FTransform::Identity;
	//cached guid in the hierarchy
	FGuid CachedHierarchyGuid;
protected:

	bool TrailOrSpaceHasChanged();
	void ExitOffsetMode();
	bool BindingHasChanged(const FGuid& InBinding, const USceneComponent* InComponent, UE::AIE::FSequencerTransformDependencies& InDependencies);

protected: 
	FGuid BindingID;
	TWeakObjectPtr<UMovieSceneTrack> MainTrack;
	UE::AIE::FSequencerTransformDependencies TransformDependencies;

	FTransform OffsetTransform;
	//transforms calculated from evaluating control rigs and actors
	TSharedPtr<UE::AIE::FArrayOfTransforms> CalculatedArrayOfTransforms;
	//calculated transforms plus any offset + parent
	TSharedPtr<UE::AIE::FArrayOfTransforms> ArrayOfTransforms;
	//parent space transforms (optional)
	TSharedPtr<UE::AIE::FArrayOfTransforms> ParentArrayOfTransforms;
	const FCurrentFramesInfo* CurrentFramesInfo;

	//optional parent space
	FOptionalParentSpace ParentSpace;
	//cached drawing data
	mutable FDrawCacheData CachedDrawData;
	//old cached draw data used to draw the trail when we are updating it
	FDrawCacheData PreviousCachedDrawData;
	//cached color values for heat map
	TArray<FLinearColor> CachedHeatMap;


	class FMovieSceneTransformTrailState : public FTrail::FMotionTrailState
	{
	public:
		FGuid BindingID;
		TWeakObjectPtr<UMovieSceneTrack> MainTrack;
		TWeakPtr<ISequencer> WeakSequencer;
		FLinearColor Color;
		EMotionTrailTrailStyle PinnedStyle;
		FTransform OffsetTransform;
		FOptionalParentSpace ParentSpace;
		void SaveFromTrail(const FMovieSceneTransformTrail* InTrail);
		void SetToTrail(FMovieSceneTransformTrail* InTrail) const;
		 
	};
	friend class FMovieSceneTransformTrailState;
};


class FMovieSceneComponentTransformTrail : public FMovieSceneTransformTrail
{
public:

	FMovieSceneComponentTransformTrail(const FGuid& InBindingID, USceneComponent* InComponent, const bool bInIsVisible, TWeakObjectPtr<UMovieSceneTrack> InWeakTrack, TSharedPtr<ISequencer> InSequencer)
		: FMovieSceneTransformTrail(InBindingID, InComponent, bInIsVisible, InWeakTrack, InSequencer),
		Component(InComponent)
	{
	}

	//new
	virtual void UpdateNeedsEvaluation(const FGuid& InTrailGuid, FSequencerTrailHierarchy* SequencerHierarchy) override;
	virtual FText GetName() const override;
	virtual TUniquePtr<FMotionTrailState> GetMotionTrailState() const;
	virtual bool HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap);


private:
	TWeakObjectPtr<USceneComponent> Component;
	// Begin FMovieSceneTransformTrail interface
	virtual bool StartTracking() override;
	virtual bool ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset) override;
	virtual bool EndTracking() override;

protected:
	virtual bool HandleAltClick(FEditorViewportClient* InViewportClient, HNewMotionTrailProxy* Proxy, FInputClick Click) override;

	// End FMovieSceneTransformTrail interface
	class FMovieSceneComponentTransformTrailState : public FMovieSceneTransformTrailState
	{
	public:
		TWeakObjectPtr<USceneComponent> Component;
		virtual void RestoreTrail(FTrailHierarchy* TrailHierarchy) override;
	};
private:
	bool bStartTracking = false;

	friend class FMovieSceneComponentTransformTrailState;
};

class FMovieSceneSocketTransformTrail : public FMovieSceneTransformTrail
{
public:

	FMovieSceneSocketTransformTrail(const FGuid& InBindingID, USceneComponent* InComponent, const FName& InSocketName,
		 const bool bInIsVisible, TWeakObjectPtr<UMovieSceneTrack> InWeakTrack,
		TSharedPtr<ISequencer> InSequencer)
		: FMovieSceneTransformTrail(InBindingID, InComponent, bInIsVisible, InWeakTrack, InSequencer),
		Component(InComponent),  SocketName(InSocketName)
	{
	}

	//new
	virtual void UpdateNeedsEvaluation(const FGuid& InTrailGuid, FSequencerTrailHierarchy* SequencerHierarchy) override;
	virtual FText GetName() const override;
	virtual TUniquePtr<FMotionTrailState> GetMotionTrailState() const;
	virtual bool HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap);


private:
	// Begin FMovieSceneTransformTrail interface
	virtual bool StartTracking() override;
	virtual bool ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset) override;
	virtual bool EndTracking() override;
	virtual void DrawHUD(const FSceneView* View, FCanvas* Canvas) override;

	virtual bool HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, FInputClick Click) override;
	virtual bool IsAnythingSelected() const override;
	virtual bool IsAnythingSelected(FVector& OutVectorPosition, FQuat& OutRotation)const override;

protected:

	// End FMovieSceneTransformTrail interface
	TWeakObjectPtr<USceneComponent> Component;
	FName SocketName;

protected:
	class FMovieSceneSocketTransformTrailState : public FMovieSceneTransformTrailState
	{
	public:
		TWeakObjectPtr<USceneComponent> Component;
		FName SocketName;
		virtual void RestoreTrail(FTrailHierarchy* TrailHierarchy) override;
	};
};

class FMovieSceneControlRigTransformTrail : public FMovieSceneTransformTrail
{
public:

	FMovieSceneControlRigTransformTrail(const FGuid& InBindingID, USceneComponent* SceneComponent,  const bool bInIsVisible, TWeakObjectPtr<UMovieSceneTrack> InWeakTrack,
		TSharedPtr<ISequencer> InSequencer, const FName& InControlName, TSharedPtr<UE::AIE::FArrayOfTransforms>& InParentArrayOfTransforms);

	virtual void UpdateNeedsEvaluation(const FGuid& InTrailGuid, FSequencerTrailHierarchy* SequencerHierarchy) override;
	virtual FText GetName() const override;
	virtual TUniquePtr<FMotionTrailState> GetMotionTrailState() const;

	virtual bool StartTracking() override;
	virtual bool ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset) override;
	virtual bool EndTracking() override;
	virtual int32 GetChannelOffset() const override;

protected:
	virtual bool HandleAltClick(FEditorViewportClient* InViewportClient, HNewMotionTrailProxy* Proxy, FInputClick Click) override;

private:
	bool bUseKeysForTrajectory = false; //on when interatively moving
	bool bStartTracking = false;

	FName ControlName;

private:
	class FMovieSceneControlRigTransformTrailState : public FMovieSceneTransformTrailState
	{
	public:
		TWeakObjectPtr<USkeletalMeshComponent> Owner;
		FName ControlName;
		virtual void RestoreTrail(FTrailHierarchy* TrailHierarchy) override;

	};

};

} // namespace MovieScene
} // namespace UE
