// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTrailHierarchy.h"

#include "Editor.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "LevelEditorViewport.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "LevelSequence.h"
#include "MotionTrailTool.h"
#include "Tools/MotionTrailOptions.h"
#include "LevelSequenceCameraSettings.h"
#include "MovieSceneTransformTrail.h"
#include "IControlRigObjectBinding.h"
#include "ActorForWorldTransforms.h"
#include "MovieSceneSequencePlaybackSettings.h"
#include "MovieSceneToolHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "Sequencer/MotionTrailMovieSceneKey.h"
#include "ControlRigObjectBinding.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Selection.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor.h"
#include "EditorModeManager.h"


namespace UE
{
namespace SequencerAnimTools
{

void FColorState::Setup(FTrailHierarchy* TrailHierarchy)
{
	Options = UMotionTrailToolOptions::GetTrailOptions();
	if(Options)
	{
		SequencerTime = TrailHierarchy->GetLocalTime();
		TicksPerFrame = TrailHierarchy->GetFramesPerFrame();
		StartFrame = TrailHierarchy->GetCurrentFramesInfo() ? TrailHierarchy->GetCurrentFramesInfo()->CurrentFrameTimes.StartFrame : TrailHierarchy->GetViewFrameRange().GetLowerBoundValue();
	}
}

EMotionTrailTrailStyle FColorState::GetStyle() const
{
	if (Options)
	{
		return Options->TrailStyle;
	}
	return PinnedStyle;

	/* Pinned style doesn't matter for now but may come back
	if (bIsPinned)
	{
		return PinnedStyle;
	}
	if (Options)
	{
		return Options->TrailStyle;
	}
	return PinnedStyle;
	*/
}

void FColorState::ReadyForTrail(bool bInIsPinned, EMotionTrailTrailStyle InPinnedStyle)
{
	bFirstFrame = true;
	bIsPinned = bInIsPinned;
	PinnedStyle = InPinnedStyle;
	CalculatedColor = FColor(0xffffff);
}

FFrameNumber FSequencerTrailHierarchy::GetLocalTime() const
{
	if(TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return Sequencer->GetLocalTime().Time.GetFrame();
	}
	return FFrameNumber(0);
}

	
using namespace UE::AIE;
TArray<TUniquePtr <FTrail::FMotionTrailState >> FSequencerTrailHierarchy::PreviouslyPinnedTrails;


FSequencerTrailHierarchy::FSequencerTrailHierarchy(TWeakPtr<ISequencer> InWeakSequencer)
	: FTrailHierarchy()
	, WeakSequencer(InWeakSequencer)
	, ObjectsTracked()
	, ControlsTracked()
	, HierarchyRenderer(MakeUnique<FTrailHierarchyRenderer>(this, UMotionTrailToolOptions::GetTrailOptions()))
	, OnActorAddedToSequencerHandle()
	, OnSelectionChangedHandle()
	, OnViewOptionsChangedHandle()
	, ControlRigDelegateHandles()
	, SavedWidgetMode(UE::Widget::WM_Translate)
	, SavedCoordSystem(COORD_Local)
{
	TrailCategory = ETrailCategory::Transform | ETrailCategory::ControlRig;
}
void FSequencerTrailHierarchy::Initialize()
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}
	UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions();
	if (!Settings)
	{
		return;
	}
	TArray<FGuid> SequencerSelectedObjects;
	Sequencer->GetSelectedObjects(SequencerSelectedObjects);
	UpdateSequencerBindings(SequencerSelectedObjects,
		[this](UObject* Object, FTrail*, FGuid Guid) {
		VisibilityManager.Selected.Add(Guid);
	});
	
	OnSelectionChangedHandle = Sequencer->GetSelectionChangedObjectGuids().AddLambda([this](TArray<FGuid> NewSelection)
	{
	
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		check(Sequencer);
		TSet<FGuid> OldSelected = VisibilityManager.Selected;
		TSet<FGuid> NewSelected;
		
		auto SetVisibleFunc = [this, &NewSelected](UObject* Object, FTrail* TrailPtr, FGuid Guid) {
			NewSelected.Add(Guid);
		};

		UpdateSequencerBindings(NewSelection, SetVisibleFunc);
		for (FGuid Guid : OldSelected)
		{
			if (NewSelected.Find(Guid) == nullptr)
			{
				RemoveTrailIfNotAlwaysVisible(Guid);
			}
		}
		VisibilityManager.Selected = NewSelected;

	});

	OnViewOptionsChangedHandle = Settings->OnDisplayPropertyChanged.AddLambda([this, Settings](FName PropertyName) {
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, EvalsPerFrame))
		{
			for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
			{
				GuidTrailPair.Value->ForceEvaluateNextTick();
			}
		}
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, TrailStyle) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, DefaultColor) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, TimePreColor) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, TimePostColor) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, DashPreColor) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, DashPostColor)
			)
		{
			for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
			{
				GuidTrailPair.Value->ClearCachedData();
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowSelectedTrails))
		{
			if (Settings->bShowSelectedTrails == false)
			{
				TArray<FGuid> TrailsToRemove; //if trail is not visible we just get rid of it
				// <parent, current>
				for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
				{
					if(GetVisibilityManager().IsTrailAlwaysVisible(GuidTrailPair.Key) == false)
					{
						TrailsToRemove.Add(GuidTrailPair.Key);
					}
				}	
				for (const FGuid& Key : TrailsToRemove)
				{
					RemoveTrail(Key);
				}
			}
			else
			{
				TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
				if (Sequencer)
				{
					TArray<FGuid> SequencerSelectedObjects;
					Sequencer->GetSelectedObjects(SequencerSelectedObjects);
					UpdateSequencerBindings(SequencerSelectedObjects,
						[this](UObject* Object, FTrail*, FGuid Guid) {
							VisibilityManager.Selected.Add(Guid);
						});
				}
			}
		}
	});

	OnObjectsReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddLambda([this](const TMap<UObject*, UObject*>& ReplacementMap)
	{
		for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
		{
			if (GuidTrailPair.Value.IsValid())
			{
				GuidTrailPair.Value->HandleObjectsChanged(ReplacementMap);
			}
		}
	});

	GEngine->OnLevelActorAdded().AddRaw(this, &FSequencerTrailHierarchy::OnActorChangedSomehow);
	GEngine->OnLevelActorDeleted().AddRaw(this, &FSequencerTrailHierarchy::OnActorChangedSomehow);
	GEngine->OnActorMoved().AddRaw(this, &FSequencerTrailHierarchy::OnActorChangedSomehow);
	GEngine->OnActorsMoved().AddRaw(this, &FSequencerTrailHierarchy::OnActorsChangedSomehow);
	//GEditor->RegisterForUndo(this);

	RegisterMotionTrailOptionDelegates();
	
	if (PreviouslyPinnedTrails.Num() > 0)
	{
		for (TUniquePtr <FTrail::FMotionTrailState >& PinnedState : PreviouslyPinnedTrails)
		{
			if (PinnedState.IsValid())
			{
				PinnedState->RestoreTrail(this);
			}
		}
		PreviouslyPinnedTrails.Reset();
	}
}

void FSequencerTrailHierarchy::Destroy()
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer)
	{
		Sequencer->OnActorAddedToSequencer().Remove(OnActorAddedToSequencerHandle);
		Sequencer->GetSelectionChangedObjectGuids().Remove(OnSelectionChangedHandle);
		if (UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions())
		{
			Settings->OnDisplayPropertyChanged.Remove(OnViewOptionsChangedHandle);
		}
	}

	for (const TPair<UMovieSceneControlRigParameterTrack*, FControlRigDelegateHandles>& SectionHandlesPair : ControlRigDelegateHandles)
	{
		UMovieSceneControlRigParameterTrack* Track = (SectionHandlesPair.Key);
		if (Track && Track->GetControlRig())
		{
			URigHierarchy* RigHierarchy = Track->GetControlRig()->GetHierarchy();
			Track->GetControlRig()->ControlSelected().Remove(SectionHandlesPair.Value.OnControlSelected);
			RigHierarchy->OnModified().Remove(SectionHandlesPair.Value.OnHierarchyModified);
		}
	}

	PreviouslyPinnedTrails.Reset();
	if (GetAllTrails().Num() > 0)
	{
		
		for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
		{
			if (VisibilityManager.IsTrailAlwaysVisible(GuidTrailPair.Key)) //if not always visible then it's selected
			{
				TUniquePtr<FTrail::FMotionTrailState> State = GuidTrailPair.Value->GetMotionTrailState();
				if (State.IsValid())
				{
					PreviouslyPinnedTrails.Add(MoveTemp(State));
				}
			}
		}
	}

	ObjectsTracked.Reset();
	ControlsTracked.Reset();
	SocketsTracked.Reset();
	AllTrails.Reset();

	GEngine->OnLevelActorAdded().RemoveAll(this);
	GEngine->OnLevelActorDeleted().RemoveAll(this);
	GEngine->OnActorMoved().RemoveAll(this);
	GEngine->OnActorsMoved().RemoveAll(this);
	//GEditor->UnregisterForUndo(this);
	VisibilityManager.Reset();
	UnRegisterMotionTrailOptionDelegates();
	FCoreUObjectDelegates::OnObjectsReplaced.Remove(OnObjectsReplacedHandle);

	if (UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions())
	{
		Settings->ResetPinnedItems();
	}
}

void FSequencerTrailHierarchy::OnActorChangedSomehow(AActor* InActor)
{
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		GuidTrailPair.Value->ActorChanged(InActor);
	}
}

void FSequencerTrailHierarchy::OnActorsChangedSomehow(TArray<AActor*>& InActors)
{
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		for (AActor* InActor : InActors)
		{
			GuidTrailPair.Value->ActorChanged(InActor);
		}
	}
}

void FSequencerTrailHierarchy::CalculateEvalRangeArray()
{
	TicksPerSegment = GetFramesPerFrame();
	CurrentFramesInfo.SetViewRange(TickViewRange, TickEvalRange == TickViewRange);
	if (LastTicksPerSegment != TicksPerSegment || TickEvalRange != LastTickEvalRange)
	{	
		LastTicksPerSegment = TicksPerSegment;
		LastTickEvalRange = TickEvalRange;
		for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
		{
			GuidTrailPair.Value->ForceEvaluateNextTick();
		}
		CurrentFramesInfo.SetUpFrameTimes(TickEvalRange, 
			TicksPerSegment);
		
	}
}

FFrameNumber FSequencerTrailHierarchy::GetFramesPerSegment() const
{
	return GetFramesPerFrame();
}

const FCurrentFramesInfo* FSequencerTrailHierarchy::GetCurrentFramesInfo() const
{
	return &CurrentFramesInfo;
}

FFrameNumber FSequencerTrailHierarchy::GetFramesPerFrame() const
{
	FFrameNumber FramesPerTick(0);
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		FramesPerTick = FFrameRate::TransformTime(FFrameNumber(1), Sequencer->GetFocusedDisplayRate(),
			Sequencer->GetFocusedTickResolution()).RoundToFrame();
	}
	return FramesPerTick;
}

void FSequencerTrailHierarchy::RemoveTrail(const FGuid& Key)
{
	FTrailHierarchy::RemoveTrail(Key);
	if (UObject* const* FoundObject = ObjectsTracked.FindKey(Key))
	{
		ObjectsTracked.Remove(*FoundObject);
	}
	else
	{
		for (TPair<USceneComponent*, TMap<FName, FGuid>>& CompMapPair : SocketsTracked)
		{
			if (const FName* FoundBone = CompMapPair.Value.FindKey(Key))
			{
				CompMapPair.Value.Remove(*FoundBone);
				return;
			}
		}
		for (TPair<UControlRig*, FControlMapAndTransforms>& CompMapPair : ControlsTracked)
		{
			if (const FName* FoundControl = CompMapPair.Value.NameToTrail.FindKey(Key))
			{
				CompMapPair.Value.NameToTrail.Remove(*FoundControl);
				return;
			}
		}
	}
}

bool FSequencerTrailHierarchy::IsTrailEvaluating(const FGuid& InGuid, bool bIndirectOnly) const
{
	if (EvaluatingTrails.Contains(InGuid))
	{
		if (bIndirectOnly)
		{
			if (const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(InGuid))
			{
				if (Trail->Get()->IsTracking()) //direct if moving a key/offset
				{
					return false;
				}
			}
		}
		return true;
	}
	return false;
}

struct FTrailControlTransforms
{
	FName ControlName;
	FGuid ElementGuid;
	TArray<FTransform> Transforms;
};

void FSequencerTrailHierarchy::EvaluateSequencerAndSetTransforms()
{
	if (EvaluatingActors.Num() > 0 || EvaluatingControlRigs.Num() > 0)
	{
		if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
		{
			TSet<FFrameNumber> MustHaveTimes;
			TSet<const UMovieSceneTrack*> DependentTracks;
			MustHaveTimes.Add(TickEvalRange.GetLowerBoundValue().Value);
			MustHaveTimes.Add(TickEvalRange.GetUpperBoundValue().Value);
			FFrameNumber CurrentFrame = SequencerPtr->GetLocalTime().Time.GetFrame();

			if (TickViewRange.GetLowerBoundValue().Value != TickEvalRange.GetLowerBoundValue().Value)
			{
				MustHaveTimes.Add(TickViewRange.GetLowerBoundValue().Value);
			}
			if (TickViewRange.GetUpperBoundValue().Value != TickEvalRange.GetUpperBoundValue().Value)
			{
				MustHaveTimes.Add(TickViewRange.GetUpperBoundValue().Value);
			}
			for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
			{
				if (EvaluatingTrails.Contains(GuidTrailPair.Key))
				{
					GuidTrailPair.Value->AddImportantTimes(MustHaveTimes);
					const UE::AIE::FSequencerTransformDependencies& TransformDependencies = (static_cast<FMovieSceneTransformTrail*>(GuidTrailPair.Value.Get()))->GetTransformDependencies();
					for (const TPair<TWeakObjectPtr<UMovieSceneTrack>,FGuid>& Track : TransformDependencies.Tracks)
					{
						if (Track.Key.IsValid())
						{
							DependentTracks.Add(Track.Key.Get());
						}
					}
				}
				else
				{
					GuidTrailPair.Value->ClearCachedData();
				}
			}
			CurrentFramesInfo.AddMustHaveTimes(MustHaveTimes, CurrentFrame);

			bool bKeepCalculating = false;

			UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
			if (World && CurrentFramesInfo.CurrentFrameTimes.NumFrames > 0)
			{
				bKeepCalculating = CurrentFramesInfo.KeepCalculating();

				const TArray<int32>& IndicesToCalcluate = CurrentFramesInfo.IndicesToCalculate();
				if (IndicesToCalcluate.Num() > 0)
				{
					UE::AIE::FEvalHelpers::CalculateWorldTransforms(World, SequencerPtr.Get(), CurrentFramesInfo.CurrentFrameTimes,
						IndicesToCalcluate, EvaluatingActors, EvaluatingControlRigs);
					UE::AIE::FEvalHelpers::EvaluateSequencer(World, SequencerPtr.Get(), DependentTracks);
				}
			}
			
			const TRange<FFrameNumber> Range = TickEvalRange;
			const TArray<int32>& IndicesToCalcluate = CurrentFramesInfo.IndicesToCalculate();
			if (CurrentFramesInfo.TransformIndices.Num() > 0 || IndicesToCalcluate.Num() > 0)
			{
				for (const FGuid& CurGuid : EvaluatingTrails)
				{
					if (const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(CurGuid))
					{
						(*Trail)->UpdateFinished(Range, IndicesToCalcluate, (bKeepCalculating == false));
					}
				}
			}

			if (bKeepCalculating == false)
			{
				EvaluatingActors.Reset();
				EvaluatingControlRigs.Reset();
				EvaluatingTrails.Reset();
			}
		}
	}
}

void FSequencerTrailHierarchy::EvaluateActor(const FGuid& InGuid, FActorForWorldTransforms& Actor, TSharedPtr<FArrayOfTransforms>& WorldTransforms, TSharedPtr<FArrayOfTransforms>& ParentWorldTransforms)
{
	if (EvaluatingTrails.Contains(InGuid) == false)
	{
		if (const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(InGuid))
		{
			(*Trail)->HasStartedEvaluating();
		}
	}

	//something changed so we reset and recalculate our cache
	CurrentFramesInfo.Reset();

	FActorAndWorldTransforms ActorAndWorldTransforms(WorldTransforms, ParentWorldTransforms);
	ActorAndWorldTransforms.Actor = Actor;
	ActorAndWorldTransforms.SetNumOfTransforms(CurrentFramesInfo.CurrentFrameTimes.NumFrames);
	EvaluatingActors.Add(ActorAndWorldTransforms);
	EvaluatingTrails.Add(InGuid);

}

void FSequencerTrailHierarchy::EvaluateControlRig(const FGuid& InGuid, UControlRig* ControlRig, const FName& ControlName, TSharedPtr<FArrayOfTransforms>& WorldTransforms)
{
	if (EvaluatingTrails.Contains(InGuid) == false)
	{
		if (const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(InGuid))
		{
			(*Trail)->HasStartedEvaluating();
		}
	}
	//something changed so we reset
	CurrentFramesInfo.Reset();

	WorldTransforms->SetNum(CurrentFramesInfo.CurrentFrameTimes.NumFrames);
	if (FControlRigAndWorldTransforms* Existing = EvaluatingControlRigs.Find(ControlRig))
	{
		Existing->SetNumOfTransforms(CurrentFramesInfo.CurrentFrameTimes.NumFrames);
		TSharedPtr<FArrayOfTransforms>& ExistingArray = Existing->ControlAndWorldTransforms.FindOrAdd(ControlName);
		ExistingArray = WorldTransforms;
	}
	else
	{
		if (FControlMapAndTransforms* Map = ControlsTracked.Find(ControlRig))
		{
			FControlRigAndWorldTransforms ControlRigAndWorldTransforms;
			ControlRigAndWorldTransforms.ControlRig = ControlRig;
			ControlRigAndWorldTransforms.ParentTransforms = Map->ArrayOfTransforms;
			ControlRigAndWorldTransforms.ParentTransforms->SetNum(CurrentFramesInfo.CurrentFrameTimes.NumFrames);
			ControlRigAndWorldTransforms.ControlAndWorldTransforms.Add(ControlName, WorldTransforms);
			EvaluatingControlRigs.Add(ControlRig, ControlRigAndWorldTransforms);
		}
	}
	EvaluatingTrails.Add(InGuid);
}

void FSequencerTrailHierarchy::Update()
{
	const FDateTime UpdateStartTime = FDateTime::Now();

	//set up the range
	UpdateViewAndEvalRange();
	//remove dead trails and let us know if they need to get updated
	FTrailHierarchy::Update(); 

	//update new ones
	EvaluateSequencerAndSetTransforms();

	const FTimespan UpdateTimespan = FDateTime::Now() - UpdateStartTime;
	TimingStats.Add("FSequencerTrailHierarchy::Update", UpdateTimespan);
}

void FSequencerTrailHierarchy::OnBindingVisibilityStateChanged(UObject* BoundObject, const EBindingVisibilityState VisibilityState)
{
	auto UpdateTrailVisibilityState = [this, VisibilityState](const FGuid& Guid) {
		if (VisibilityState == EBindingVisibilityState::AlwaysVisible)
		{
			VisibilityManager.AlwaysVisible.Add(Guid);
		}
		else if (VisibilityState == EBindingVisibilityState::VisibleWhenSelected)
		{
			VisibilityManager.AlwaysVisible.Remove(Guid);
		}
		};

	if (ObjectsTracked.Contains(BoundObject))
	{
		UpdateTrailVisibilityState(ObjectsTracked[BoundObject]);
	}

	USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(BoundObject);
	if (!SkelMeshComp)
	{
		return;
	}

	for (TPair<UControlRig*, FControlMapAndTransforms>& CompMapPair : ControlsTracked)
	{
		if (UControlRig* ControlRig = CompMapPair.Key)
		{
			if (ControlRig->GetObjectBinding() && ControlRig->GetObjectBinding()->GetBoundObject() == FControlRigObjectBinding::GetBindableObject(BoundObject))
			{
				for (TPair<FName, FGuid> NameGuid : CompMapPair.Value.NameToTrail)
				{
					UpdateTrailVisibilityState(NameGuid.Value);
				}
				break;
			}
		}
	}
}

void FSequencerTrailHierarchy::UpdateSequencerBindings(const TArray<FGuid>& SequencerBindings, TFunctionRef<void(UObject*, FTrail*, FGuid)> OnUpdated)
{
	const FDateTime StartTime = FDateTime::Now();
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
	if (!ensure(Sequencer))
	{
		return;
	}

	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence() ? Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene() : nullptr;
	if (!ensure(MovieScene))
	{
		return;
	}

	TArray<AActor*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);
	TArray<USceneComponent*> SelectedSceneComponents;
	GEditor->GetSelectedActors()->GetSelectedObjects<USceneComponent>(SelectedSceneComponents);

	for (FGuid BindingGuid : SequencerBindings)
	{
		bool bAddedControlRig = false;
		const TArray<UMovieSceneTrack*> ControlRigTracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), BindingGuid);
		for (UMovieSceneTrack* Track : ControlRigTracks)
		{
			if (UMovieSceneControlRigParameterTrack* CRParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
			{
				for (TWeakObjectPtr<> BoundObject : Sequencer->FindBoundObjects(BindingGuid, Sequencer->GetFocusedTemplateID()))
				{
					if (!BoundObject.IsValid())
					{
						continue;
					}

					USkeletalMeshComponent* BoundComponent = Cast<USkeletalMeshComponent>(BoundObject.Get());
					if (AActor* BoundActor = Cast<AActor>(BoundObject.Get()))
					{
						BoundComponent = BoundActor->FindComponentByClass<USkeletalMeshComponent>();
					}

					if (!BoundComponent || !BoundComponent->GetSkeletalMeshAsset() || !BoundComponent->GetSkeletalMeshAsset()->GetSkeleton())
					{
						continue;
					};

					//if we have a selected control rig don't add the transform track also, makes pinning messy
					if (!ControlRigDelegateHandles.Contains(CRParameterTrack))
					{
						RegisterControlRigDelegates(BoundComponent, CRParameterTrack);
						UControlRig* ControlRig = CRParameterTrack->GetControlRig();
						if (ControlRig)
						{
							TArray<FName> Selected = ControlRig->CurrentControlSelection();
							for (const FName& ControlName : Selected)
							{
								AddControlRigTrail(BoundComponent, ControlRig, CRParameterTrack, ControlName);
								bAddedControlRig = true;
							}
						}
					}
				}
			}
		} // if ControlRigParameterTrack
		if (bAddedControlRig)
		{
			ClearSelection();
			continue;
		}
		if (UMovieScene3DTransformTrack* TransformTrack = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(BindingGuid))
		{
			for (TWeakObjectPtr<> BoundObject : Sequencer->FindBoundObjects(BindingGuid, Sequencer->GetFocusedTemplateID()))
			{
				if (!BoundObject.IsValid())
				{
					continue;
				}

				/*
				//if using old trails don't add new ones.
				if (CVarUseOldSequencerMotionTrails->GetBool() == true)
				{
					continue;
				}
				*/

				USceneComponent* BoundComponent = Cast<USceneComponent>(BoundObject.Get());
				if (AActor* BoundActor = Cast<AActor>(BoundObject.Get()))
				{
					if (SelectedActors.Contains(BoundActor) == false)
					{
						continue;
					}
					BoundComponent = BoundActor->GetRootComponent();
				}
				else if (SelectedSceneComponents.Contains(BoundComponent) == false)
				{
					continue;
				}

				if (!ObjectsTracked.Contains(BoundComponent))
				{
					AddComponentToHierarchy(BindingGuid, BoundComponent, TransformTrack);
				}

				if (!ObjectsTracked.Contains(BoundComponent))
				{
					continue;
				}

				if (AllTrails.Contains(ObjectsTracked[BoundComponent]) && AllTrails[ObjectsTracked[BoundComponent]].IsValid())
				{
					OnUpdated(BoundComponent, AllTrails[ObjectsTracked[BoundComponent]].Get(), ObjectsTracked[BoundComponent]);
				}
			}
		} // if TransformTrack
		if (UMovieSceneSkeletalAnimationTrack* AnimTrack = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieSceneSkeletalAnimationTrack>(BindingGuid))
		{
			for (TWeakObjectPtr<> BoundObject : Sequencer->FindBoundObjects(BindingGuid, Sequencer->GetFocusedTemplateID()))
			{
				if (!BoundObject.IsValid())
				{
					continue;
				}
				/*
				if (CVarUseOldSequencerMotionTrails->GetBool() == true)
				{
					continue;
				}
				*/

				USkeletalMeshComponent* BoundComponent = Cast<USkeletalMeshComponent>(BoundObject.Get());
				if (AActor* BoundActor = Cast<AActor>(BoundObject.Get()))
				{
					if (SelectedActors.Contains(BoundActor) == false)
					{
						continue;
					}
					BoundComponent = BoundActor->FindComponentByClass<USkeletalMeshComponent>();
				}
				else if (SelectedSceneComponents.Contains(BoundComponent) == false)
				{
					continue;
				}

				if (!BoundComponent || !BoundComponent->GetSkeletalMeshAsset() || !BoundComponent->GetSkeletalMeshAsset()->GetSkeleton())
				{
					continue;
				}



				if (!ObjectsTracked.Contains(BoundComponent))
				{
					if (UMovieScene3DTransformTrack* TransformTrack = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(BindingGuid))
					{
						AddComponentToHierarchy(BindingGuid, BoundComponent, TransformTrack);
					}
				}
			}
		}
	}
	const FTimespan Timespan = FDateTime::Now() - StartTime;
	TimingStats.Add("FSequencerTrailHierarchy::UpdateSequencerBindings", Timespan);
}


bool FSequencerTrailHierarchy::CheckForChanges()
{
	bool bHasChange = false;
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		if (Sequencer->GetFocusedMovieSceneSequence() && Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene())
		{
			bHasChange = LastValidMovieSceneGuid != Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetSignature();
			LastValidMovieSceneGuid = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetSignature();
		}
	}
	return bHasChange;
}

void FSequencerTrailHierarchy::UpdateViewAndEvalRange()
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{

		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();
		if (UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions())
		{
			//if range is selected then eval and view is selected, otherwise eval is full and view maybe ticksbefore,after
			if (Settings->MotionTrailRange == EMotionTrailRange::SelectionRange && Sequencer->GetSelectionRange().IsEmpty() == false)
			{
				TickEvalRange = Sequencer->GetSelectionRange();
				TickViewRange = TickEvalRange;
			}
			else
			{
				TOptional<TRange<FFrameNumber>> OptionalRange = Sequencer->GetSubSequenceRange();
				TickEvalRange = OptionalRange.IsSet() ? OptionalRange.GetValue() : Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
				if (Settings->MotionTrailRange == EMotionTrailRange::SpecifiedRange)
				{
					FFrameTime SequenceTime = Sequencer->GetLocalTime().Time;
					const FFrameNumber TicksBefore = FFrameRate::TransformTime(FFrameNumber(UMotionTrailToolOptions::GetTrailOptions()->FramesBefore), DisplayRate, TickResolution).FloorToFrame();
					const FFrameNumber TicksAfter = FFrameRate::TransformTime(FFrameNumber(UMotionTrailToolOptions::GetTrailOptions()->FramesAfter), DisplayRate, TickResolution).FloorToFrame();
					TickViewRange = TRange<FFrameNumber>(SequenceTime.GetFrame() - TicksBefore, SequenceTime.GetFrame() + TicksAfter);
					if (TickViewRange.GetLowerBoundValue() < TickEvalRange.GetLowerBoundValue())
					{
						TickViewRange.SetLowerBoundValue(TickEvalRange.GetLowerBoundValue());
					}
					if (TickViewRange.GetUpperBoundValue() > TickEvalRange.GetUpperBoundValue())
					{
						TickViewRange.SetUpperBoundValue(TickEvalRange.GetUpperBoundValue());
					}
				}
				else //full range
				{
					TickViewRange = TickEvalRange;
				}
			}
		}
	}
}

FGuid FSequencerTrailHierarchy::AddComponentToHierarchy(const FGuid& InBindingGuid, USceneComponent* CompToAdd, UMovieScene3DTransformTrack* TransformTrack)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	check(Sequencer);
	
	const FGuid CurTrailGuid = ObjectsTracked.FindOrAdd(CompToAdd, FGuid::NewGuid());

	TUniquePtr<FTrail> CurTrail = MakeUnique<FMovieSceneComponentTransformTrail>(InBindingGuid, CompToAdd, false, TransformTrack, Sequencer);
	if (AllTrails.Contains(ObjectsTracked[CompToAdd])) 
	{
		AllTrails.Remove(ObjectsTracked[CompToAdd]);
	}
	CurTrail->ForceEvaluateNextTick();

	AddTrail(ObjectsTracked[CompToAdd], MoveTemp(CurTrail));

	ClearSelection();

	return CurTrailGuid;
}

FGuid FSequencerTrailHierarchy::AddControlRigTrail(USkeletalMeshComponent* Component, UControlRig* ControlRig, UMovieSceneControlRigParameterTrack* CRParameterTrack,const FName& ControlName)
{
	FGuid NewGuid;
	if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlName))
	{
		if (ControlElement->IsAnimationChannel())
		{
			return NewGuid; // no shape
		}
	}
	if (ControlsTracked.Find(ControlRig) == nullptr)
	{
		FControlMapAndTransforms MapAndTransforms;
		MapAndTransforms.ArrayOfTransforms = MakeShared<FArrayOfTransforms>();
		
		ControlsTracked.Add(ControlRig, MapAndTransforms);
	}
	if (!ControlsTracked[ControlRig].NameToTrail.Contains(ControlName))
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer)
		{
			//find binding id
			TArray<UMovieSceneControlRigParameterTrack*> Tracks;
			const UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
			const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
			FGuid BindingID;
			for (const FMovieSceneBinding& Binding : Bindings)
			{
				TArray<UMovieSceneTrack*> FoundTracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
				for (UMovieSceneTrack* Track : FoundTracks)
				{
					if (UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
					{
						if (CRTrack->GetControlRig() == ControlRig)
						{
							BindingID = Binding.GetObjectGuid();
							break;
						}
					}
				}
			}
			const FGuid ControlGuid = ControlsTracked.Find(ControlRig)->NameToTrail.FindOrAdd(ControlName, FGuid::NewGuid());
			TUniquePtr<FTrail> CurTrail = MakeUnique<FMovieSceneControlRigTransformTrail>(BindingID,Component, false, CRParameterTrack, Sequencer,
				ControlName, ControlsTracked[ControlRig].ArrayOfTransforms);
			if (AllTrails.Contains(ControlGuid))
			{
				AllTrails.Remove(ControlGuid);
				VisibilityManager.ControlSelected.Remove(ControlGuid);
			}
			AddTrail(ControlGuid, MoveTemp(CurTrail));
			VisibilityManager.ControlSelected.Add(ControlGuid);
			NewGuid = ControlGuid;
		}
	}
	else
	{
		const FGuid* ControlGuid = ControlsTracked.Find(ControlRig)->NameToTrail.Find(ControlName);
		if (ControlGuid != nullptr)
		{
			VisibilityManager.ControlSelected.Add(*ControlGuid);
			NewGuid = *ControlGuid;
		}
	}
	return NewGuid;
}

void FSequencerTrailHierarchy::ClearSelection()
{
	const bool bShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	if (!bShiftDown)
	{
		for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
		{
			GuidTrailPair.Value->SelectNone();
		}
	}
}

void FSequencerTrailHierarchy::RegisterControlRigDelegates(USkeletalMeshComponent* Component, UMovieSceneControlRigParameterTrack* CRParameterTrack)
{
	if (CRParameterTrack == nullptr)
	{
		return;
	}
	UControlRig* ControlRig = CRParameterTrack->GetControlRig();
	if (Component == nullptr || ControlRig == nullptr || ControlRig->GetHierarchy() == nullptr)
	{
		return;
	}
	TWeakObjectPtr<USkeletalMeshComponent> WeakComponent = Component;
	TWeakObjectPtr<UMovieSceneControlRigParameterTrack> WeakTrack = CRParameterTrack;

	FControlRigDelegateHandles& DelegateHandles = ControlRigDelegateHandles.Add(CRParameterTrack);

	DelegateHandles.OnControlSelected = ControlRig->ControlSelected().AddLambda([this, WeakComponent, WeakTrack](UControlRig* ControlRig, FRigControlElement* ControlElement, bool bSelected)
		{
			TStrongObjectPtr<USkeletalMeshComponent> ComponentPtr = WeakComponent.Pin();
			TStrongObjectPtr<UMovieSceneControlRigParameterTrack> TrackPtr = WeakTrack.Pin();
			TSharedPtr <ISequencer> SequencerPtr = WeakSequencer.Pin();
			if (!ComponentPtr || !TrackPtr || !SequencerPtr)
			{
				return;
			}
				
			if (ControlElement->Settings.ControlType != ERigControlType::Transform &&
				ControlElement->Settings.ControlType != ERigControlType::TransformNoScale &&
				ControlElement->Settings.ControlType != ERigControlType::EulerTransform)
			{
				return;
			}

			if (bSelected)
			{
				AddControlRigTrail(ComponentPtr.Get(), ControlRig, TrackPtr.Get(), ControlElement->GetFName());
			}

			if (ControlsTracked.Find(ControlRig) != nullptr)
			{
				if (ControlsTracked[ControlRig].NameToTrail.Contains(ControlElement->GetFName()))
				{
					if (bSelected == false)
					{
						const FGuid TrailGuid = ControlsTracked[ControlRig].NameToTrail[ControlElement->GetFName()];
						VisibilityManager.ControlSelected.Remove(TrailGuid);
						RemoveTrailIfNotAlwaysVisible(TrailGuid);
					}
				}
			}

			//check to see if the seleced control rig is sill selected
			TArray<FGuid> TrailsToRemove;
			for (TPair<UControlRig*, FControlMapAndTransforms>& CompMapPair : ControlsTracked)
			{			
				UControlRig* TrackedControlRig = CompMapPair.Key;
				for (TPair<FName, FGuid> NameGuid : CompMapPair.Value.NameToTrail)
				{
					if (TrackedControlRig->IsControlSelected(NameGuid.Key) == false)
					{
						FGuid TrailGuid = NameGuid.Value;
						TrailsToRemove.Add(TrailGuid);
					}
				}
			}
			for (const FGuid TrailGuid: TrailsToRemove)
			{
				VisibilityManager.ControlSelected.Remove(TrailGuid);
				RemoveTrailIfNotAlwaysVisible(TrailGuid);
			}

			ClearSelection();
		}

			
	);
	
	//just use ControlRig and not WeakControlRig since it's only used as a TMap Key
	URigHierarchy* RigHierarchy = ControlRig->GetHierarchy();
	DelegateHandles.OnHierarchyModified = RigHierarchy->OnModified().AddLambda(
		[this, ControlRig](ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject) 
		{
		
			TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
			if (Sequencer.IsValid() == false)
			{
				return;
			}
			
			const FRigControlElement* ControlElement = Cast<FRigControlElement>(InSubject.Element); 
			if(ControlElement == nullptr)
			{
				return;
			}
			
			if(InNotif == ERigHierarchyNotification::ElementRemoved)
			{
				if (!ControlsTracked.Contains(ControlRig) || !ControlsTracked[ControlRig].NameToTrail.Contains(ControlElement->GetFName())) // We only care about controls
				{
					return;
				}
				
				const FGuid TrailGuid = ControlsTracked[ControlRig].NameToTrail[ControlElement->GetFName()];
				RemoveTrail(TrailGuid);
			}
            else if(InNotif == ERigHierarchyNotification::ElementRenamed)
            {
            	const FName OldName = InHierarchy->GetPreviousName(ControlElement->GetKey());

            	if (!ControlsTracked.Contains(ControlRig) || !ControlsTracked[ControlRig].NameToTrail.Contains(OldName))
				{
					return;
				}

				const FGuid TempTrailGuid = ControlsTracked[ControlRig].NameToTrail[OldName];
				ControlsTracked[ControlRig].NameToTrail.Remove(OldName);
				ControlsTracked[ControlRig].NameToTrail.Add(ControlElement->GetFName(), TempTrailGuid);
            }
        }
    );
}

void FSequencerTrailHierarchy::RegisterMotionTrailOptionDelegates()
{
	if (UMotionTrailToolOptions* TrailOptions = UMotionTrailToolOptions::GetTrailOptions())
	{
		TrailOptions->OnPinSelection.AddRaw(this, &FSequencerTrailHierarchy::OnPinSelection);
		TrailOptions->OnUnPinSelection.AddRaw(this, &FSequencerTrailHierarchy::OnUnPinSelection);
		TrailOptions->OnPinComponent.AddRaw(this, &FSequencerTrailHierarchy::OnPinComponent);
		TrailOptions->OnDeletePinned.AddRaw(this, &FSequencerTrailHierarchy::OnDeletePinned);
		TrailOptions->OnDeleteAllPinned.AddRaw(this, &FSequencerTrailHierarchy::OnDeleteAllPinned);
		TrailOptions->OnPutPinnedInSpace.AddRaw(this, &FSequencerTrailHierarchy::OnPutPinnedInSpace);
		TrailOptions->OnSetLinearColor.AddRaw(this, &FSequencerTrailHierarchy::OnSetLinearColor);
		TrailOptions->OnSetHasOffset.AddRaw(this, &FSequencerTrailHierarchy::OnSetHasOffset);
	}
}

void FSequencerTrailHierarchy::UnRegisterMotionTrailOptionDelegates()
{
	if (UMotionTrailToolOptions* TrailOptions = UMotionTrailToolOptions::GetTrailOptions())
	{
		TrailOptions->OnPinSelection.RemoveAll(this);
		TrailOptions->OnUnPinSelection.RemoveAll(this);
		TrailOptions->OnPinComponent.RemoveAll(this);
		TrailOptions->OnDeletePinned.RemoveAll(this);
		TrailOptions->OnDeleteAllPinned.RemoveAll(this);
		TrailOptions->OnPutPinnedInSpace.RemoveAll(this);
		TrailOptions->OnSetLinearColor.RemoveAll(this);
		TrailOptions->OnSetHasOffset.RemoveAll(this);
	}
}

void FSequencerTrailHierarchy::PinTrail(FGuid InGuid)
{
	if (const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(InGuid))
	{
		if (UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions())
		{
			if (Settings->GetNumPinned() >= Settings->MaxNumberPinned)
			{
				UE_LOGF(LogTemp, Warning, "MotionTrails: Cannot Pin trail %ls Max number reached. Please delete pinned trail if you want to add this one.", *(*Trail)->GetName().ToString());
				return;
			}

			VisibilityManager.SetTrailAlwaysVisible(InGuid, true);
			if ((*Trail)->GetDrawInfo())
			{
				(*Trail)->GetDrawInfo()->SetColor(Settings->DefaultColor);
				(*Trail)->GetDrawInfo()->SetStyle(Settings->TrailStyle);

				UMotionTrailToolOptions::FPinnedTrail PinnedTrail;
				PinnedTrail.TrailGuid = InGuid;
				PinnedTrail.TrailName = (*Trail)->GetName();
				PinnedTrail.TrailColor = (*Trail)->GetDrawInfo()->GetColor();
				PinnedTrail.bHasOffset = (*Trail)->HasOffsetTransform();
				PinnedTrail.Category = ETrailCategory::Transform | ETrailCategory::ControlRig;
				Settings->AddPinned(PinnedTrail);
			}
		}
	}
}

void FSequencerTrailHierarchy::OnPinSelection(ETrailCategory Category)
{
	if (!EnumHasAnyFlags(Category, ETrailCategory::Transform | ETrailCategory::ControlRig))
	{
		return;
	}

	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		if (VisibilityManager.IsTrailAlwaysVisible(GuidTrailPair.Key) == false) //if not always visible then it's selected
		{
			PinTrail(GuidTrailPair.Key);
		}
	}
}

void FSequencerTrailHierarchy::OnUnPinSelection(ETrailCategory Category)
{
	if (!EnumHasAnyFlags(Category, ETrailCategory::Transform | ETrailCategory::ControlRig))
	{
		return;
	}

	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		if (VisibilityManager.IsTrailAlwaysVisible(GuidTrailPair.Key) == true 
			&& (VisibilityManager.ControlSelected.Contains(GuidTrailPair.Key) || VisibilityManager.Selected.Contains(GuidTrailPair.Key)))
		{
			if (UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions())
			{
				int32 Index = Settings->GetIndexFromGuid(GuidTrailPair.Key);
				if (Index != INDEX_NONE)
				{
					Settings->DeletePinned(Index);
				}
			}
			else
			{
				VisibilityManager.SetTrailAlwaysVisible(GuidTrailPair.Key, false);
			}
		}
	}
}

void FSequencerTrailHierarchy::OnPinComponent(USceneComponent* InSceneComponent, FName InSocketName)
{
	PinComponent(InSceneComponent, InSocketName);
}

FGuid FSequencerTrailHierarchy::PinComponent(USceneComponent* InSceneComponent, FName InSocketName)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	check(Sequencer);
	TArray<FMovieSceneObjectBindingID> Bindings;
	Sequencer->GetEvaluationState()->FilterObjectBindings(InSceneComponent, *Sequencer, &Bindings);
	if (Bindings.Num() <= 0)
	{
		AActor* ParentActor = InSceneComponent->GetOwner();
		Sequencer->GetEvaluationState()->FilterObjectBindings(ParentActor, *Sequencer, &Bindings);

	}
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	TWeakObjectPtr<UMovieSceneTrack> MainTrack = nullptr;
	if (Bindings.Num() > 0)
	{
		const FMovieSceneObjectBindingID Binding = Bindings[0];
		TArray<UMovieSceneTrack*> FoundCRTracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetGuid(), NAME_None);
		TArray<UMovieSceneTrack*> FoundSkelAnimTracks = MovieScene->FindTracks(UMovieSceneSkeletalAnimationTrack::StaticClass(), Binding.GetGuid(), NAME_None);
		TArray<UMovieSceneTrack*> FoundTransformTracks = MovieScene->FindTracks(UMovieScene3DTransformTrack::StaticClass(), Binding.GetGuid(), NAME_None);

		if (FoundCRTracks.Num() > 0)
		{
			MainTrack = FoundCRTracks[0];
		}
		else if (FoundSkelAnimTracks.Num() > 0 && FoundSkelAnimTracks[0]->GetAllSections().Num() > 0)
		{
			MainTrack = FoundSkelAnimTracks[0];
		}
		else if (FoundTransformTracks.Num() > 0)
		{
			MainTrack = FoundTransformTracks[0];
		}
	}
	if (MainTrack.IsValid() == false) //no binding or no tracks so bail
	{
		UE_LOGF(LogTemp, Warning, "MotionTrails: No binding or tracks for pinned component. Please make sure it is added to Sequencer.");
		return FGuid();
	}

	TMap<FName, FGuid>& BoneNameGuidPair = SocketsTracked.FindOrAdd(InSceneComponent);
	const FGuid* Guid = BoneNameGuidPair.Find(InSocketName);
	if (Guid)
	{
		if (AllTrails.Contains(*Guid))
		{
			AllTrails.Remove(*Guid);
		}
		VisibilityManager.AlwaysVisible.Remove(*Guid);
	}
	
	const FMovieSceneObjectBindingID Binding = Bindings[0];
	TUniquePtr<FTrail> CurTrail = MakeUnique<FMovieSceneSocketTransformTrail>(Binding.GetGuid(), InSceneComponent, InSocketName, false, MainTrack, Sequencer);
	
	CurTrail->ForceEvaluateNextTick();
	const FGuid BoneGuid = FGuid::NewGuid();
	BoneNameGuidPair.Add(InSocketName, BoneGuid);
	VisibilityManager.AlwaysVisible.Add(BoneGuid);
	if (UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions())
	{
		if (CurTrail->GetDrawInfo())
		{
			CurTrail->GetDrawInfo()->SetColor(Settings->DefaultColor);
			CurTrail->GetDrawInfo()->SetStyle(Settings->TrailStyle);

			UMotionTrailToolOptions::FPinnedTrail Trail;
			Trail.TrailGuid = BoneGuid;
			Trail.TrailName = CurTrail->GetName();
			Trail.TrailColor = CurTrail->GetDrawInfo()->GetColor();
			Trail.bHasOffset = CurTrail->HasOffsetTransform();
			Trail.Category = ETrailCategory::Transform | ETrailCategory::ControlRig;
			Settings->AddPinned(Trail);
		}
	}

	AddTrail(BoneGuid, MoveTemp(CurTrail));
	return BoneGuid;
}

void FSequencerTrailHierarchy::OnDeletePinned(FGuid InGuid)
{
	VisibilityManager.SetTrailAlwaysVisible(InGuid, false);
	if (VisibilityManager.ControlSelected.Contains(InGuid) == false && VisibilityManager.Selected.Contains(InGuid) == false)
	{
		RemoveTrail(InGuid);
	}
}

void FSequencerTrailHierarchy::OnDeleteAllPinned(ETrailCategory Category)
{
	if (!EnumHasAnyFlags(Category, ETrailCategory::Transform | ETrailCategory::ControlRig))
	{
		return;
	}

	TArray<FGuid> AlwaysVisibleTrails = VisibilityManager.AlwaysVisible.Array();
	for (const FGuid& Guid: AlwaysVisibleTrails)
	{
		if (VisibilityManager.ControlSelected.Contains(Guid) == false && VisibilityManager.Selected.Contains(Guid) == false)
		{
			RemoveTrail(Guid);
		}
	}
	VisibilityManager.AlwaysVisible.Reset();
}

void FSequencerTrailHierarchy::OnPutPinnedInSpace(FGuid InGuid, AActor* InActor, FName InComponentName)
{
	if (const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(InGuid))
	{
		if ((*Trail).IsValid())
		{
			if (InActor)
			{
				(*Trail)->SetSpace(InActor, InComponentName);
				(*Trail)->ForceEvaluateNextTick();

			}
			else
			{
				(*Trail)->ClearSpace();
				(*Trail)->ForceEvaluateNextTick();
			}
		}
	}
}

void FSequencerTrailHierarchy::OnSetLinearColor(FGuid InGuid, FLinearColor Color)
{
	if (const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(InGuid))
	{
		if ((*Trail).IsValid())
		{
			if (FTrajectoryDrawInfo* CurDrawInfo = (*Trail)->GetDrawInfo())
			{
				CurDrawInfo->SetColor(Color);
			}
		}
	}
}

void FSequencerTrailHierarchy::OnSetHasOffset(FGuid InGuid, bool bOffset)
{
	if (const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(InGuid))
	{
		if ((*Trail).IsValid())
		{
			if (bOffset != (*Trail)->GetOffsetMode())
			{
				if (bOffset == false)
				{
					GLevelEditorModeTools().SetWidgetMode(SavedWidgetMode);
					GLevelEditorModeTools().SetCoordSystem(SavedCoordSystem);
					(*Trail)->ClearOffsetTransform();
				}
				else
				{
					SavedWidgetMode = GLevelEditorModeTools().GetWidgetMode();
					SavedCoordSystem = GLevelEditorModeTools().GetCoordSystem();
					GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Translate);
					GLevelEditorModeTools().SetCoordSystem(COORD_Local);
					(*Trail)->SetOffsetMode();
				}
			}
			else
			{
				if (bOffset == false)
				{
					(*Trail)->ClearOffsetTransform();
				}
			}
		}
	}
}

} // namespace MovieScene
} // namespace UE


