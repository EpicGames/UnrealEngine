// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrailHierarchy.h"
#include "ISequencer.h"
#include "Tools/EvaluateSequencerTools.h"
#include "UnrealWidgetFwd.h"

namespace UE::SequencerAnimTools { class FTrail; }
namespace UE::SequencerAnimTools { class FTrailEvaluateTimes; }


class ISequencer;
class USceneComponent;
class USkeletalMeshComponent;
class USkeleton;
class UMovieScene;
class UMovieSceneSection;
class UMovieScene3DTransformTrack;
class UMovieSceneControlRigParameterTrack;
class UMovieSceneControlRigParameterSection;
class UControlRig;
struct FRigHierarchyContainer;


namespace UE::Widget
{
	enum EWidgetMode;
};
namespace UE
{
namespace SequencerAnimTools
{

enum class EBindingVisibilityState
{
	AlwaysVisible,
	VisibleWhenSelected
};

class FSequencerTrailHierarchy : public FTrailHierarchy
{
public:
	FSequencerTrailHierarchy(TWeakPtr<ISequencer> InWeakSequencer);
	
	//New
	void EvaluateActor(const FGuid& InKey, FActorForWorldTransforms& InActor,TSharedPtr<UE::AIE::FArrayOfTransforms>& WorldTransforms, TSharedPtr<UE::AIE::FArrayOfTransforms>& ParentWorldTransforms);
	void EvaluateControlRig(const FGuid& InKey, UControlRig* InControlRig, const FName& InControlName, TSharedPtr<UE::AIE::FArrayOfTransforms>& WorldTransforms);

	// FTrailHierarchy interface
	virtual void Initialize() override;
	virtual void Destroy() override;
	virtual ITrailHierarchyRenderer* GetRenderer() const override { return HierarchyRenderer.Get(); }
	virtual FFrameNumber GetFramesPerFrame() const override;
	virtual FFrameNumber GetFramesPerSegment() const override;
	virtual const FCurrentFramesInfo* GetCurrentFramesInfo() const override;
	virtual bool CheckForChanges() override;

	virtual void RemoveTrail(const FGuid& InTrailGuid) override;
	virtual void Update() override;
	virtual void CalculateEvalRangeArray() override;
	virtual bool IsTrailEvaluating(const FGuid& InTrailGuid, bool bIndirectlyOnly) const override;

	// End FTrailHierarchy interface

	void OnBindingVisibilityStateChanged(UObject* BoundObject, const EBindingVisibilityState VisibilityState);
	//called when added moved, deleted, added, all of which may need a refresh of a motion trail
	void OnActorChangedSomehow(AActor* InActor);
	void OnActorsChangedSomehow(TArray<AActor*>& InActors);	

	virtual FFrameNumber GetLocalTime() const override;
	
	FGuid AddComponentToHierarchy(const FGuid& InBindingGuid, USceneComponent* CompToAdd, UMovieScene3DTransformTrack* TransformTrack);
	FGuid AddControlRigTrail(USkeletalMeshComponent* Component, UControlRig* ControlRig, UMovieSceneControlRigParameterTrack* CRTrack, const FName& ControlName);
	FGuid PinComponent(USceneComponent* InSceneComponent, FName InSocketName);

private:
	


	void UpdateSequencerBindings(const TArray<FGuid>& SequencerBindings, TFunctionRef<void(UObject*, FTrail*, FGuid)> OnUpdated);
	void UpdateViewAndEvalRange();

	void RegisterControlRigDelegates(USkeletalMeshComponent* Component, UMovieSceneControlRigParameterTrack* CRParameterTrack);

	TWeakPtr<ISequencer> WeakSequencer;
	TMap<UObject*, FGuid> ObjectsTracked;
	TMap<USceneComponent*, TMap<FName, FGuid>> SocketsTracked;


	struct FControlMapAndTransforms
	{
		TMap < FName, FGuid> NameToTrail;
		TSharedPtr<UE::AIE::FArrayOfTransforms> ArrayOfTransforms;
	};
	TMap<UControlRig*, FControlMapAndTransforms> ControlsTracked;

	// TODO: components can have multiple rigs so make this a map from sections to controls instead. However, this is only part of a larger problem of handling blending

	TUniquePtr<FTrailHierarchyRenderer> HierarchyRenderer;

	FDelegateHandle OnActorAddedToSequencerHandle;
	FDelegateHandle OnSelectionChangedHandle;
	FDelegateHandle OnViewOptionsChangedHandle;
	FDelegateHandle OnObjectsReplacedHandle;

	struct FControlRigDelegateHandles
	{
		FDelegateHandle OnHierarchyModified;
		FDelegateHandle OnControlSelected;
	};
	TMap<UMovieSceneControlRigParameterTrack*, FControlRigDelegateHandles> ControlRigDelegateHandles;
	
private:
	//new evaluate setup.. this is what we are evaluating...
	TArray<UE::AIE::FActorAndWorldTransforms> EvaluatingActors;
	TMap<UControlRig*, UE::AIE::FControlRigAndWorldTransforms> EvaluatingControlRigs;
	TSet<FGuid> EvaluatingTrails;

	//current frames
	FCurrentFramesInfo  CurrentFramesInfo;
	
	void EvaluateSequencerAndSetTransforms();

	static TArray<TUniquePtr <FTrail::FMotionTrailState >> PreviouslyPinnedTrails;
public:
	void PinTrail(FGuid InGuid);

private:
	//cached guid to check for updates
	FGuid LastValidMovieSceneGuid;
	// for offset mode
	UE::Widget::EWidgetMode SavedWidgetMode;
	ECoordSystem SavedCoordSystem;
protected: //callbacks
	void ClearSelection(); 
	void RegisterMotionTrailOptionDelegates();
	void UnRegisterMotionTrailOptionDelegates();
	void OnPinSelection(ETrailCategory Category);
	void OnUnPinSelection(ETrailCategory Category);
	void OnDeleteAllPinned(ETrailCategory Category);
	void OnPinComponent(USceneComponent* InSceneComponent, FName SocketName);
	void OnDeletePinned(FGuid InGuid);
	void OnPutPinnedInSpace(FGuid InGuid, AActor* InActor, FName InComponentName);
	void OnSetLinearColor(FGuid InGuid, FLinearColor Color);
	void OnSetHasOffset(FGuid InGuid, bool bOffset);

};



} // namespace MovieScene
} // namespace UE
