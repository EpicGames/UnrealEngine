// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerAnimEditPivotTool.h"
#include "BaseGizmos/TransformProxy.h"
#include "ISequencer.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "IControlRigObjectBinding.h"
#include "InteractiveGizmoManager.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "Framework/Application/SlateApplication.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"

#include "InteractiveToolManager.h"
// for raycast into World

#include "ConstraintsManager.h"
#include "EditorModeManager.h"
#include "Misc/QualifiedFrameTime.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneTimeUnit.h"
#include "ControlRig.h"
#include "ControlRigComponent.h"
#include "ControlRigSequencerEditorLibrary.h"
#include "EditorInteractiveGizmoManager.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "ILevelSequenceEditorToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "SceneView.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Rigs/FKControlRig.h"
#include "Selection.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Viewports/InViewportUIDragOperation.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Styling/StyleColors.h"
#include "SLevelViewport.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "EditorGizmos/TransformGizmo.h"
#include "EditMode/ControlRigEditMode.h"

#define LOCTEXT_NAMESPACE "SequencerAnimTools"

void FEditPivotCommands::RegisterCommands()
{
	UI_COMMAND(ResetPivot, "Reset Pivot To Original", "Reset pivot back to original location", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control|EModifierKey::Shift|EModifierKey::Alt, EKeys::G));
	UI_COMMAND(ToggleFreePivot, "Toggle Edit/Pose", "Toggle gizmo to move freely or to pivot the selection", EUserInterfaceActionType::Button, FInputChord(EKeys::P));
}


FSavedMappings USequencerPivotTool::SavedPivotLocations;
FLastSelectedObjects USequencerPivotTool::LastSelectedObjects;

namespace UE::AIE::PivotTool
{

	TArray<FName> CurrentControlSelection(UControlRig* InControlRig) 
	{
		TArray<FName> Names;
		if (InControlRig)
		{
			// use this version after 5.7.1 is done, expose FControlRigEditMode::GetRigElementsForSettingTransforms
			//TArray<FRigElementKey> SelectedRigElements = FControlRigEditMode::GetRigElementsForSettingTransforms(InControlRig, true /*bSelectionOnly*/, false /*bIncludeChannels*/);
			/*for (const FRigElementKey& Key : SelectedRigElements)
			{
				Names.Add(Key.Name);
			}
			*/

			TArray<FName> SelectedControls = InControlRig->CurrentControlSelection();
			for (const FName& Name : SelectedControls)
			{
				if (FRigControlElement* Element = InControlRig->FindControl(Name))
				{
					if (Element->IsAnimationChannel() == false)
					{
						Names.Add(Name);
					}
				}
			}
		}

		return Names;
	}
	
	USceneComponent* GetBoundComponent(const UControlRig* InControlRig)
	{
		const TSharedPtr<IControlRigObjectBinding>& ObjectBinding = InControlRig ? InControlRig->GetObjectBinding() : nullptr;
		return ObjectBinding.IsValid() ? Cast<USceneComponent>(ObjectBinding->GetBoundObject()) : nullptr;
	}
	
	FTransform GetBoundComponentTransform(const UControlRig* InControlRig)
	{
		USceneComponent* HostingComponent = GetBoundComponent(InControlRig);
		return HostingComponent ? HostingComponent->GetComponentTransform() : FTransform::Identity;
	}
	
	//includes global + hosting transform
	FTransform GetControlWorldSpaceTransform(UControlRig* ControlRig, const FName& Control)
	{
		FTransform WorldTransform = FTransform::Identity;
		if (ControlRig && ControlRig->GetHierarchy())
		{
			if (FRigTransformElement* ControlElement = ControlRig->FindControl(Control))
			{
				const FTransform ParentTransform = GetBoundComponentTransform(ControlRig);
				//Use GetTransform() not GetControlGlobalTransform
				WorldTransform = ControlRig->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal);
				WorldTransform = WorldTransform * ParentTransform;
			}
		}
		return WorldTransform;
	}
}

static void GetControlRigsAndSequencer(TArray<TWeakObjectPtr<UControlRig>>& ControlRigs, TWeakPtr<ISequencer>& SequencerPtr, ULevelSequence** LevelSequence)
{
	*LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetFocusedLevelSequence();
	//if getting sequencer from level sequence need to use the current(master), not the focused
	ULevelSequence* SequencerLevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	if (*LevelSequence && SequencerLevelSequence)
	{
		IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(SequencerLevelSequence, false);
		ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
		SequencerPtr = LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
		if (SequencerPtr.IsValid())
		{
			//use editt mode (which must be active for pivot tool) to get 
			TArray<UControlRig*> TempControlRigs;
			FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
			if (ControlRigEditMode)
			{
				TempControlRigs = ControlRigEditMode->GetControlRigsArray(false /*bIsVisible*/); //get all may want to pivot non-visible
			}
			for (UControlRig* ControlRig : TempControlRigs)
			{
				ControlRigs.Add(ControlRig);
			}
		}
	}
}
bool USequencerPivotToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	//only start if we have anything selected  or we selected somethign last time
	//which will then get selected later once the tool starts.
	if (USequencerPivotTool::LastSelectedObjects.LastSelectedControlRigs.Num() > 0 || USequencerPivotTool::LastSelectedObjects.LastSelectedActors.Num() > 0)
	{
		return true;
	}
	ULevelSequence* LevelSequence;
	TArray<TWeakObjectPtr<UControlRig>> ControlRigs;
	TWeakPtr<ISequencer> SequencerPtr;
	GetControlRigsAndSequencer(ControlRigs, SequencerPtr, &LevelSequence);

	for (TWeakObjectPtr<UControlRig> ControlRig : ControlRigs)
	{
		if (ControlRig.IsValid() && ControlRig->CurrentControlSelection().Num() > 0)
		{
			return true;
		}
	}

	TArray<AActor*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);
	if (SelectedActors.Num() > 0)
	{
		return true;
	}
	
	return false;
}

UInteractiveTool* USequencerPivotToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	USequencerPivotTool* NewTool = NewObject<USequencerPivotTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World, SceneState.GizmoManager);
	return NewTool;
}
/*
* ControlRig/Actor Mappings
*/

TOptional<FTransform> FControlRigMappings::GetWorldTransform(const FName& Name) const
{
	
	TOptional<FTransform> WorldTransform;
	if (ControlRig.IsValid() && ControlRig->GetHierarchy())
	{
		if (FRigTransformElement* ControlElement = ControlRig->FindControl(Name))
		{
			const FTransform* LocalTransform = PivotTransforms.Find(Name);
			if (LocalTransform)
			{
				const FTransform ToWorldTransform = UE::AIE::PivotTool::GetBoundComponentTransform(ControlRig.Get());
				//Use GetTransform() not GetControlGlobalTransform
				FTransform CurrentTransform = ControlRig->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal);
				WorldTransform = (*LocalTransform) * CurrentTransform * ToWorldTransform;
			}
		}
	}
	return WorldTransform;
}

void FControlRigMappings::SetFromWorldTransform(const FName& Name, const FTransform& WorldTransform)
{
	if (ControlRig.IsValid() && ControlRig->GetHierarchy())
	{
		if (FRigTransformElement* ControlElement = ControlRig->FindControl(Name))
		{
			FTransform ToWorldTransform = UE::AIE::PivotTool::GetBoundComponentTransform(ControlRig.Get());
			FTransform GlobalTransform = ControlRig->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal) * ToWorldTransform;

			FTransform LocalTransform = WorldTransform.GetRelativeTransform(GlobalTransform);
			PivotTransforms.Add(Name, LocalTransform);
		}
	}
}

TArray<FName> FControlRigMappings::GetAllControls() const
{
	TArray<FName> Controls;
	for (const TPair<FName, FTransform>& Pair : PivotTransforms)
	{
		Controls.Add(Pair.Key);
	}
	return Controls;
}

void FControlRigMappings::SelectControls()
{
	if (ControlRig.IsValid())
	{
		for (TPair<FName, FTransform>& Pair : PivotTransforms)
		{
			ControlRig->SelectControl(Pair.Key, true);
		}
	}
}

bool FControlRigMappings::IsAnyControlDeselected() const
{
	if (ControlRig.IsValid())
	{
		for (const TPair<FName, FTransform>& Pair : PivotTransforms)
		{
			if (ControlRig->IsControlSelected(Pair.Key) == false)
			{
				return true;
			}
		}
		return false;
	}
	return true;
}

TOptional<FTransform> FActorMappings::GetWorldTransform() const
{
	TOptional<FTransform> Transform;
	if (Actor.IsValid())
	{
		FTransform ActorTransform = Actor->ActorToWorld();
		Transform = PivotTransform * ActorTransform;
	}
	return Transform;
}

void FActorMappings::SetFromWorldTransform( const FTransform& WorldTransform)
{
	if (Actor.IsValid())
	{
		FTransform ActorTransform = Actor->ActorToWorld();
		FTransform LocalTransform = WorldTransform.GetRelativeTransform(ActorTransform);
		PivotTransform = LocalTransform;
	}

}
void FActorMappings::SelectActors()
{
	if (Actor.IsValid())
	{
		GEditor->SelectActor(Actor.Get(), true, true);
		GEditor->NoteSelectionChange();
	}
}


/*
* Proxy
*/

struct HSequencerPivotProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	HSequencerPivotProxy(EHitProxyPriority InPriority = HPP_Foreground) :
		HHitProxy(InPriority)
	{}
};

IMPLEMENT_HIT_PROXY(HSequencerPivotProxy, HHitProxy);

/*
* Tool
*/

void USequencerPivotTool::SetWorld(UWorld* World, UInteractiveGizmoManager* InGizmoManager)
{
	TargetWorld = World;
	GizmoManager = InGizmoManager;
}

void USequencerPivotTool::Setup()
{
	//when entered we check to see if shift is pressed this can change where we set the pivot on start or reset
	FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
	bShiftPressedWhenStarted = KeyState.IsShiftDown();
	bCtrlPressedWhenStarted = KeyState.IsControlDown();
	if (bCtrlPressedWhenStarted)
	{
		bShiftPressedWhenStarted = false; //for now turn off shift which means we pivot from the last if control is also presseed
	}


	UInteractiveTool::Setup();

	ClickBehavior = NewObject<USingleClickInputBehavior>(this);
	ClickBehavior->Initialize(this);
	AddInputBehavior(ClickBehavior);

	TransformProxy = NewObject<UTransformProxy>(this);

	TransformProxy->OnTransformChanged.AddUObject(this, &USequencerPivotTool::GizmoTransformChanged);
	TransformProxy->OnBeginTransformEdit.AddUObject(this, &USequencerPivotTool::GizmoTransformStarted);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &USequencerPivotTool::GizmoTransformEnded);

	FSequencerAnimToolHelpers::FGizmoData GizmoData;
	GizmoData.Owner = this;
	GizmoData.TransformProxy = TransformProxy;
	GizmoData.ToolManager = GetToolManager();
	GizmoData.GizmoManager = GizmoManager;
	GizmoData.InstanceIdentifier = TEXT("PivotToolGizmoIdentifier");
	FSequencerAnimToolHelpers::CreateGizmo(GizmoData, TransformGizmo, TRSGizmo);
	//turn default widget(transform gizmo) off when entering this mode if not off
	if (GLevelEditorModeTools().GetShowWidget())
	{
		bSetShowWidget = true;
		GLevelEditorModeTools().SetShowWidget(false); 
	}

	GetControlRigsAndSequencer(ControlRigs, SequencerPtr, &LevelSequence);

	UpdateTransformAndSelectionOnEntering();

	TArray<AActor*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);
	Actors.SetNum(0);
	for (AActor* Actor : SelectedActors)
	{
		Actors.Add(Actor);
	}

	UpdateGizmoTransform();
	UpdateGizmoVisibility();
	//we get delegates last since we may select something above
	for (TWeakObjectPtr<UControlRig>& ControlRig : ControlRigs)
	{
		if (ControlRig.IsValid())
		{
			ControlRig->ControlSelected().AddUObject(this, &USequencerPivotTool::HandleControlSelected);
		}
	}

	OnEditorSelectionChangedHandle = USelection::SelectionChangedEvent.AddUObject(this, &USequencerPivotTool::OnEditorSelectionChanged);

	SaveLastSelected();
	SavePivotTransforms();

	CommandBindings = MakeShareable(new FUICommandList);

	const FEditPivotCommands& Commands = FEditPivotCommands::Get();

	CommandBindings->MapAction(
		Commands.ResetPivot,
		FExecuteAction::CreateUObject(this, &USequencerPivotTool::ResetPivot)
		);
	CommandBindings->MapAction(
		Commands.ToggleFreePivot,
		FExecuteAction::CreateUObject(this, &USequencerPivotTool::TogglePivotMode)
	);
	SetPivotMode(false);
	CreateAndShowPivotOverlay();
	
	Keyframer.Initialize();
	ConstraintsCache.RegisterNotifications();
}

void USequencerPivotTool::SetPivotMode(bool bVal)
{ 
	bInPivotMode = bVal;
	if (bInPivotMode == false)
	{
		FEditorModeTools& ModeManager = GLevelEditorModeTools(); //mz need to fix for persona/ animation edit mode
		ModeManager.SetCoordSystem(COORD_Local);
	}
	Keyframer.Enable(bInPivotMode);
}

void USequencerPivotTool::OnTick(float DeltaTime) 
{
	if (bGizmoBeingDragged == false)
	{
		SetGizmoBasedOnSelection(true);
		UpdateGizmoTransform();
	}
}

void USequencerPivotTool::ResetPivot()
{
	SetGizmoBasedOnSelection(false);
	UpdateGizmoTransform();
	SavePivotTransforms();
}

//Last selected is really the ones that were selected when you entered the tool
//we use this in case nothing was selected when the tool is active so that we instead select it.
void USequencerPivotTool::SaveLastSelected()
{
	LastSelectedObjects.LastSelectedControlRigs.SetNum(0);
	LastSelectedObjects.LastSelectedActors.SetNum(0);

	for (TWeakObjectPtr<UControlRig>& ControlRig : ControlRigs)
	{
		if (ControlRig.IsValid())
		{
			TArray<FName> SelectedControls = UE::AIE::PivotTool::CurrentControlSelection(ControlRig.Get());
			for (const FName& Name : SelectedControls)
			{
				FControlRigMappings Mapping;
				Mapping.ControlRig = ControlRig;
				Mapping.SetFromWorldTransform(Name, GizmoTransform);
				LastSelectedObjects.LastSelectedControlRigs.Add(Mapping);
			}
		}
	}
	for (TWeakObjectPtr<AActor>& Actor : Actors)
	{
		if (Actor.IsValid())
		{
			FActorMappings Mapping;
			Mapping.Actor = Actor;
			Mapping.SetFromWorldTransform(GizmoTransform);
			LastSelectedObjects.LastSelectedActors.Add(Mapping);
		}
	}
}

//when we enter the tool if we have things selected we get its last transform position
//if not we should have the last thing selected, and if so we select that.
void USequencerPivotTool::UpdateTransformAndSelectionOnEntering()
{
	//if shift was pressed when we started we don't use saved, this will move pivot to last object
	bool bhaveSomethingSelected = SetGizmoBasedOnSelection(!bShiftPressedWhenStarted && !bCtrlPressedWhenStarted);
	//okay nothing selected we select the last thing selected
	if (bhaveSomethingSelected == false)
	{
		for (FControlRigMappings& Mappings : LastSelectedObjects.LastSelectedControlRigs)
		{
			Mappings.SelectControls();
		}
		for (FActorMappings& ActorMappings : LastSelectedObjects.LastSelectedActors)
		{
			ActorMappings.SelectActors();

		}
		SetGizmoBasedOnSelection(!bShiftPressedWhenStarted && !bCtrlPressedWhenStarted);
	}
}

bool USequencerPivotTool::ProcessCommandBindings(const FKey Key, const bool bRepeat) const
{
	if (CommandBindings.IsValid())
	{
		FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
		return CommandBindings->ProcessCommandBindings(Key, KeyState, bRepeat);
	}
	return false;
}

bool USequencerPivotTool::SetGizmoBasedOnSelection(bool bUseSaved)
{

	GizmoTransform = FTransform::Identity;
	FVector Location(0.0f, 0.0f, 0.0f);
	ISequencer* Sequencer = SequencerPtr.Pin().Get();
	if (Sequencer == nullptr)
	{
		return false;
	}
	const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	const FFrameTime FrameTime = Sequencer->GetLocalTime().ConvertTo(TickResolution);

	bool bHaveSomethingSelected = false;
	FVector AverageLocation(0.0f);
	int NumLocations = 0;
	for (TWeakObjectPtr<UControlRig>& ControlRig : ControlRigs)
	{
		if (ControlRig.IsValid() == false)
		{
			continue;
		}
		FControlRigMappings* Mappings = SavedPivotLocations.ControlRigMappings.Find(ControlRig);
		TArray<FName> SelectedControls = UE::AIE::PivotTool::CurrentControlSelection(ControlRig.Get());

		for (const FName& Name : SelectedControls)
		{
			bHaveSomethingSelected = true;
			bool bHasSavedPivot = false;
			if (bUseSaved && Mappings)
			{
				TOptional<FTransform>WorldTransform = Mappings->GetWorldTransform(Name);
				if (WorldTransform.IsSet())
				{
					GizmoTransform = WorldTransform.GetValue();
					bHasSavedPivot = true;
				}
			}
			if (bHasSavedPivot == false)
			{
				UControlRig* RealControlRig = Cast<UControlRig>(ControlRig.Get());
				GizmoTransform  = UE::AIE::PivotTool::GetControlWorldSpaceTransform(RealControlRig, Name);

				AverageLocation += GizmoTransform.GetLocation();
				NumLocations++;
			}
		}
	}
	//now do actors, to do need to do them in order somehow if using bUseSaved.... (shift is pressed) and we have multiple

	TArray<AActor*> SelectedActors;
	GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);
	for (AActor* SelectedActor : SelectedActors)
	{
		bHaveSomethingSelected = true;
		FActorMappings* Mappings = SavedPivotLocations.ActorMappings.Find(SelectedActor);
		if (bUseSaved && Mappings)
		{

			TOptional<FTransform>WorldTransform = Mappings->GetWorldTransform();
			if (WorldTransform.IsSet())
			{
				GizmoTransform = WorldTransform.GetValue();
			}
			else
			{
				GizmoTransform = UControlRigSequencerEditorLibrary::GetActorWorldTransform(LevelSequence, SelectedActor, FrameTime.RoundToFrame(),
					EMovieSceneTimeUnit::TickResolution);
			}
		}
		else
		{
			GizmoTransform = UControlRigSequencerEditorLibrary::GetActorWorldTransform(LevelSequence,SelectedActor, FrameTime.RoundToFrame(),
				EMovieSceneTimeUnit::TickResolution);
			AverageLocation += GizmoTransform.GetLocation();
			NumLocations++;
		}
	}
	if (bCtrlPressedWhenStarted && NumLocations > 1)
	{
		AverageLocation /= (float)(NumLocations);
		GizmoTransform.SetLocation(AverageLocation);
	}
	GizmoTransform.SetScale3D(FVector(1.0f, 1.0f, 1.0f));
	return bHaveSomethingSelected;
}
void USequencerPivotTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (bSetShowWidget.IsSet() && bSetShowWidget.GetValue() == true)
	{
		GLevelEditorModeTools().SetShowWidget(true); //turn widget back on
	}
	GizmoManager->DestroyAllGizmosByOwner(this);
	RemoveDelegates();
	RemoveAndDestroyPivotOverlay();
	
	InteractionDependencies.Reset();
	ConstraintsCache.UnregisterNotifications();
	ConstraintsCache.Reset();
}

void USequencerPivotTool::DeactivateMe()
{
	//if in undo make sure we have lost selection else leave alone
	bool bDeactivate = true;
	if (GIsTransacting)
	{
		bDeactivate = false;
		for (FControlRigMappings& Mappings : LastSelectedObjects.LastSelectedControlRigs)
		{
			if (Mappings.ControlRig.IsValid())
			{
				bDeactivate = Mappings.IsAnyControlDeselected();
			}
			else
			{
				bDeactivate = true;
				break;
			}
		}

		if(bDeactivate == false)
		{ 
			TArray<AActor*> SelectedActors;
			GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);

			for (FActorMappings& ActorMappings : LastSelectedObjects.LastSelectedActors)
			{
				
				if (ActorMappings.Actor.IsValid())
				{
					if (SelectedActors.Contains(ActorMappings.Actor.Get()) == false)
					{
						bDeactivate = true;
						break;
					}
				}
				else
				{
					bDeactivate = true;
					break;
				}
			}
		}
	}
	if (bDeactivate)
	{
		if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
		{
			TSharedPtr<ILevelEditor> LevelEditorPtr = LevelEditorModule->GetLevelEditorInstance().Pin();

			if (LevelEditorPtr.IsValid())
			{
				FString ActiveToolName = LevelEditorPtr->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->GetActiveToolName(EToolSide::Left);
				if (ActiveToolName == TEXT("SequencerPivotTool"))
				{
					LevelEditorPtr->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->DeactivateTool(EToolSide::Left, EToolShutdownType::Completed);
				}
			}
		}
	}

}

void USequencerPivotTool::OnEditorSelectionChanged(UObject* NewSelection)
{
	DeactivateMe();
	/** Later may need to keep track of ordering
	USelection* Selection = Cast<USelection>(NewSelection);
	if (!Selection)
	{
		return;
	}

	TArray<UObject*> SelectedActors;
	Selection->GetSelectedObjects(AActor::StaticClass(), SelectedActors);
	*/
}

void USequencerPivotTool::HandleControlSelected(UControlRig* Subject, FRigControlElement* InControl, bool bSelected)
{
	DeactivateMe();
}

void USequencerPivotTool::RemoveDelegates()
{
	for (TWeakObjectPtr<UControlRig>& ControlRig : ControlRigs)
	{
		if (ControlRig.IsValid())
		{
			ControlRig->ControlSelected().RemoveAll(this);
		}
	}

	USelection::SelectionChangedEvent.Remove(OnEditorSelectionChangedHandle);
	OnEditorSelectionChangedHandle.Reset();
}

void USequencerPivotTool::GizmoTransformStarted(UTransformProxy* Proxy)
{
	using namespace UE::AIE::PivotTool;
	
	Keyframer.Enable(bInPivotMode);
	InteractionDependencies.Reset();
	TransformContext = FControlRigInteractionTransformContext();
	
	ControlRigDrags.SetNum(0);
	
	ISequencer* Sequencer = SequencerPtr.Pin().Get();
	if (Sequencer && LevelSequence)
	{
		const bool bHandleTransaction = GEditor && !GIsTransacting && GEditor->CanTransact() && !GEditor->IsTransactionActive();
		if (bHandleTransaction)
		{
			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("USequencerPivotToolTransaction", "Transform"));
		}
		
		for (TWeakObjectPtr<UControlRig>& WeakControlRig : ControlRigs)
		{
			PrepareRigForManipulation(WeakControlRig.Get());
		}

		for (TWeakObjectPtr<AActor>& Actor : Actors)
		{
			PrepareActorForManipulation(Actor.Get());
		}
	}

	StartDragTransform = Proxy->GetTransform();
	bGizmoBeingDragged = true;
	bManipulatorMadeChange = false;
	GizmoTransform = StartDragTransform;

	InteractionScopes.Reset();

	if(bInPivotMode)
	{
		TMap<UControlRig*, int32> RigToScopeIndex;
		for(int32 IndexA = 0; IndexA < ControlRigDrags.Num(); IndexA++)
		{
			const FControlRigSelectionDuringDrag& ControlRigDrag = ControlRigDrags[IndexA];

			// if we are hitting this for the first time
			const int32 ScopeIndex = RigToScopeIndex.FindOrAdd(ControlRigDrag.ControlRig, InteractionScopes.Num());
			if(!InteractionScopes.IsValidIndex(ScopeIndex))
			{
				// get all keys on the same control rig
				TArray<FRigElementKey> Keys = {FRigElementKey(ControlRigDrag.ControlName, ERigElementType::Control)};
				for(int32 IndexB = IndexA + 1; IndexB < ControlRigDrags.Num(); IndexB++)
				{
					if(ControlRigDrags[IndexB].ControlRig != ControlRigDrag.ControlRig)
					{
						continue;
					}
					Keys.Add(FRigElementKey(ControlRigDrags[IndexB].ControlName, ERigElementType::Control));
				}
				InteractionScopes.Emplace(MakeShareable(new FControlRigInteractionScope(ControlRigDrag.ControlRig, Keys)));
			}
		}
	}
}



//For some reason when just setting the location of the actor the gizmo was not updating, so we do the full suite of notifications to make
//sure the other systems know the actor got moved
static void SetLocation(AActor* Actor, const FVector& NewLocation,const TOptional<FQuat>& NewQuat)
{
	if (Actor != nullptr)
	{
		if (USceneComponent* RootComponent = Actor->GetRootComponent())
		{
			GEditor->BroadcastBeginObjectMovement(*Actor);
			FProperty* TransformProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName());
			FEditPropertyChain PropertyChain;
			PropertyChain.AddHead(TransformProperty);
			FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(Actor, PropertyChain);
			RootComponent->SetWorldLocation(NewLocation);
			if (NewQuat.IsSet())
			{
				RootComponent->SetWorldRotation(NewQuat.GetValue());
			}
			FPropertyChangedEvent PropertyChangedEvent(TransformProperty, EPropertyChangeType::ValueSet);
			// Broadcast Post Edit change notification, we can't call PostEditChangeProperty directly on Actor or ActorComponent from here since it wasn't pair with a proper PreEditChange
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Actor, PropertyChangedEvent);
			GUnrealEd->UpdatePivotLocationForSelection();
			GUnrealEd->SetPivotMovedIndependently(false);
			const bool bFinished = false;	// @todo gizmo: PostEditChange never called; and bFinished=true never known!!
			Actor->PostEditMove(bFinished);
			GEditor->BroadcastEndObjectMovement(*Actor);
		}
	}
}

void USequencerPivotTool::SetControlTransform(const FControlRigSelectionDuringDrag& InControlDrag)
{
	UControlRig* ControlRig = InControlDrag.ControlRig;
	if (!ControlRig || InControlDrag.ControlName == NAME_None)
	{
		return;
	}
	
	const bool bFixEulerFlips = (!ControlRig->IsAdditive() || ControlRig->IsA<UFKControlRig>()) && TransformContext.bRotation;
	UE::ControlRigEditMode::FInteractionDependencyCache* Dependencies = InteractionDependencies.Find(ControlRig);
	auto EvaluateRigIfNeeded = [Dependencies, &InControlDrag]()
	{
		// skip compensation and evaluate the rig to force notifications: auto-key and constraints updates (among others) are based on
		// UControlRig::OnControlModified being broadcast but this only happens on evaluation for additive rigs.
		// constraint compensation is disabled while manipulating in that case to avoid re-entrant evaluations
		TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
			
		if (InControlDrag.ControlRig->IsAdditive())
		{
			InControlDrag.ControlRig->Evaluate_AnyThread();
		}
		else if (Dependencies)
		{
			if (Dependencies->HasDownwardDependencies({InControlDrag.ControlName, ERigElementType::Control}) ||
				Dependencies->CheckAndUpdateParentsPoseVersion())
			{
				InControlDrag.ControlRig->Evaluate_AnyThread();
			}
		}
	};
	
	const bool bDeferAutokey = Keyframer.IsEnabled();
	const uint32 ControlHash = UTransformableControlHandle::ComputeHash(ControlRig, InControlDrag.ControlName);
	const bool bNeedsConstraintPostProcess = ConstraintsCache.HasAnyActiveConstraint(ControlHash, TargetWorld);
	static const TArray< TWeakObjectPtr<UTickableConstraint> > EmptyConstraints;
	const TArray< TWeakObjectPtr<UTickableConstraint> >& Constraints =
		bNeedsConstraintPostProcess ? ConstraintsCache.Get(ControlHash, TargetWorld) : EmptyConstraints;

	FRigControlModifiedContext Context;
	Context.bConstraintUpdate = true;
	Context.SetKey = bDeferAutokey ? EControlRigSetKey::Never : EControlRigSetKey::DoNotCare;

	USceneComponent* Component = UE::AIE::PivotTool::GetBoundComponent(ControlRig);
	const FTransform& ComponentTransform = Component ? Component->GetComponentTransform() : FTransform::Identity;
	
	const FTransform Global = InControlDrag.CurrentTransform.GetRelativeTransform(ComponentTransform);
	
	constexpr bool bNotify = true, bSetupUndo = true, bPrintPythonCommands = false;
	ControlRig->SetControlGlobalTransform(
		InControlDrag.ControlName, Global, bNotify, Context, bSetupUndo, bPrintPythonCommands, bFixEulerFlips);
	EvaluateRigIfNeeded();
				
	UE::AnimationEditMode::FControlKeyframeData KeyframeData;
	KeyframeData.LocalTransform = ControlRig->GetControlLocalTransform(InControlDrag.ControlName);
	
	if (bNeedsConstraintPostProcess)
	{
		// switch to constraint space
		const TOptional<FTransform> RelativeTransform =
			UE::TransformConstraintUtil::GetConstraintsRelativeTransform(Constraints, KeyframeData.LocalTransform, InControlDrag.CurrentTransform);
		if (RelativeTransform)
		{
			KeyframeData.LocalTransform = *RelativeTransform;
			KeyframeData.bConstraintSpace = true;
		}

		FRigControlModifiedContext LocalContext = Context;
		LocalContext.bConstraintUpdate = false;

		ControlRig->SetControlLocalTransform(InControlDrag.ControlName, KeyframeData.LocalTransform, bNotify, LocalContext, bSetupUndo, bFixEulerFlips);
		EvaluateRigIfNeeded();
	}

	if (bNeedsConstraintPostProcess || ConstraintsCache.HasAnyActiveConstraint(Component, TargetWorld))
	{
		TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(TargetWorld);
		Controller.EvaluateAllConstraints();
	}
	
	Keyframer.Store(ControlHash, MoveTemp(KeyframeData));
}

void USequencerPivotTool::GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	if (!bGizmoBeingDragged)
	{
		return;
	}
	bManipulatorMadeChange = true;
	
	FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
	bool bAlsoCalcRotation = KeyState.IsShiftDown() == false;
	GizmoTransform = Transform;
	FTransform Diff = Transform.GetRelativeTransform(StartDragTransform);
	TransformContext.bRotation = (Diff.GetRotation().IsIdentity(1e-4f) == false);
	TransformContext.bTranslation = (Diff.GetTranslation().IsNearlyZero() == false);
	
	if (TransformContext.bRotation || TransformContext.bTranslation)
	{
		int32 Index = 0;
		for (FControlRigSelectionDuringDrag& ControlDrag : ControlRigDrags)
		{
			if (!bInPivotMode)
			{
				ControlDrag.CurrentTransform = Transform;
			}
			else if (!bShiftPressedWhenStarted || Index != (ControlRigDrags.Num() - 1))
			{
				if (TransformContext.bRotation)
				{
					FVector LocDiff = ControlDrag.CurrentTransform.GetLocation() - Transform.GetLocation();
					if (LocDiff.IsNearlyZero(1e-4f) == false)
					{
						LocDiff = StartDragTransform.InverseTransformPosition(ControlDrag.CurrentTransform.GetLocation());
						FVector RotatedDiff = Diff.GetRotation().RotateVector(LocDiff);
						FVector NewLocation = StartDragTransform.TransformPosition(RotatedDiff);
						ControlDrag.CurrentTransform.SetLocation(NewLocation);
					}
					
					if (bAlsoCalcRotation)
					{
						FQuat StartDragRot = StartDragTransform.GetRotation();
						FQuat ActorDragRot = ControlDrag.CurrentTransform.GetRotation().Inverse();
						FQuat ParentRot = ActorDragRot * StartDragRot;
						ParentRot = ParentRot.Inverse();
						FQuat CurrentDragRot = Transform.GetRotation();
						FQuat OptQuat = CurrentDragRot * ParentRot;
						OptQuat.EnforceShortestArcWith(ControlDrag.CurrentTransform.GetRotation());
						ControlDrag.CurrentTransform.SetRotation(OptQuat);
					}
					
					SetControlTransform(ControlDrag);
				}
				else if (TransformContext.bTranslation)
				{
					FVector DiffTranslation = StartDragTransform.GetRotation().RotateVector(Diff.GetTranslation());
					ControlDrag.CurrentTransform.AddToTranslation(DiffTranslation);
					SetControlTransform(ControlDrag);
				}
			}
			else //last one with shift we keep locked!
			{
				SetControlTransform(ControlDrag);
			}
			
			++Index;
		}

		Index = 0;
		for (FActorSelectonDuringDrag& ActorDrag : ActorDrags)
		{
			if (bInPivotMode == false)
			{
				ActorDrag.CurrentTransform = Transform;
			}
			else if (!bShiftPressedWhenStarted || Index != (ActorDrags.Num() - 1))
			{
				if (TransformContext.bRotation)
				{
					FVector LocDiff = ActorDrag.CurrentTransform.GetLocation() - Transform.GetLocation();
					if (LocDiff.IsNearlyZero(1e-4f) == false)
					{
						LocDiff = StartDragTransform.InverseTransformPosition(ActorDrag.CurrentTransform.GetLocation());
						FVector RotatedDiff = Diff.GetRotation().RotateVector(LocDiff);
						FVector NewLocation = StartDragTransform.TransformPosition(RotatedDiff);
						ActorDrag.CurrentTransform.SetLocation(NewLocation);
						TOptional <FQuat> OptQuat;
						if (bAlsoCalcRotation)
						{
							FQuat StartDragRot = StartDragTransform.GetRotation();
							FQuat ActorDragRot = ActorDrag.CurrentTransform.GetRotation().Inverse();
							FQuat ParentRot = ActorDragRot * StartDragRot;
							ParentRot = ParentRot.Inverse();
							FQuat CurrentDragRot = Transform.GetRotation();
							OptQuat = CurrentDragRot * ParentRot;
							ActorDrag.CurrentTransform.SetRotation(OptQuat.GetValue());
						}
						SetLocation(ActorDrag.Actor, NewLocation, OptQuat);
					}
					else if (bAlsoCalcRotation)
					{
						FQuat StartDragRot = StartDragTransform.GetRotation();
						FQuat ActorDragRot = ActorDrag.CurrentTransform.GetRotation().Inverse();
						FQuat ParentRot =  ActorDragRot * StartDragRot;
						ParentRot = ParentRot.Inverse();
						FQuat CurrentDragRot = Transform.GetRotation();
						FQuat OptQuat = CurrentDragRot * ParentRot;
						OptQuat.EnforceShortestArcWith(ActorDrag.CurrentTransform.GetRotation());
						ActorDrag.CurrentTransform.SetRotation(OptQuat);
						SetLocation(ActorDrag.Actor, ActorDrag.CurrentTransform.GetLocation(), OptQuat);
					}
				}
				else if (TransformContext.bTranslation)
				{
					FVector DiffTranslation = StartDragTransform.GetRotation().RotateVector(Diff.GetTranslation());
					ActorDrag.CurrentTransform.AddToTranslation(DiffTranslation);
					const FVector NewLocation = ActorDrag.CurrentTransform.GetLocation();
					TOptional <FQuat> OptQuat;
					SetLocation(ActorDrag.Actor, NewLocation, OptQuat);
				}
			}
			else //last one with shift we keep locked!
			{
				TOptional <FQuat> OptQuat;
				SetLocation(ActorDrag.Actor, ActorDrag.CurrentTransform.GetLocation(),OptQuat);
			}
			++Index;
		}
	}
	
	StartDragTransform = Transform;
	
	UpdateBoundComponents();
}

void USequencerPivotTool::GizmoTransformEnded(UTransformProxy* Proxy)
{
	const bool bFinalizeTransaction = Transaction && bManipulatorMadeChange;
	
	bGizmoBeingDragged = false;
	bManipulatorMadeChange = false;
	
	// autokey on release
	{
		for (const TSharedPtr<FControlRigInteractionScope>& InteractionScope: InteractionScopes)
		{
			Keyframer.Apply(*InteractionScope, TransformContext);
		}
		Keyframer.Finalize(TargetWorld);
		Keyframer.Reset();
	}
	
	if (Transaction)
	{
		if (!bFinalizeTransaction)
		{
			Transaction->Cancel();
		}
		Transaction.Reset();
	}
	
	InteractionScopes.Reset();
	UpdateGizmoTransform();
	SavePivotTransforms();
	InteractionDependencies.Reset();
}

void USequencerPivotTool::SavePivotTransforms()
{
	for (FControlRigMappings& LastObject : LastSelectedObjects.LastSelectedControlRigs)
	{
		FControlRigMappings& Mappings = SavedPivotLocations.ControlRigMappings.FindOrAdd(LastObject.ControlRig);
		Mappings.ControlRig = LastObject.ControlRig;
		TArray<FName> LastNames = LastObject.GetAllControls();
		for (const FName& Name : LastNames)
		{
			Mappings.SetFromWorldTransform(Name,GizmoTransform);
		}
	}

	for (FActorMappings& LastObject : LastSelectedObjects.LastSelectedActors)
	{
		FActorMappings& Mappings = SavedPivotLocations.ActorMappings.FindOrAdd(LastObject.Actor);
		Mappings.Actor = LastObject.Actor;
		Mappings.SetFromWorldTransform(GizmoTransform);
	}
}

//currently always visibile
void USequencerPivotTool::UpdateGizmoVisibility()
{
	if (TransformGizmo)
	{
		TransformGizmo->SetVisibility(true);
	}
	
	if (TRSGizmo)
	{
		TRSGizmo->SetVisibility(true);
	}
}

void USequencerPivotTool::UpdateGizmoTransform()
{
	FEditorModeTools& ModeManager = GLevelEditorModeTools(); //mz need to fix for persona/ animation edit mode
	/* This should work but doesn't. It affects the 'highlighting' correctly but it doesn't render correctly
	#jira UE-306751
	const double Scale = (double)ModeManager.GetWidgetScale();
	const FVector3d ScaleVector(Scale, Scale, Scale);
	FTransform GizmoTransformWithScale(GizmoTransform);
	GizmoTransformWithScale.SetScale3D(ScaleVector);
	*/
	if (TransformGizmo)
	{
		//TransformGizmo->ReinitializeGizmoTransform(GizmoTransformWithScale);
		TransformGizmo->ReinitializeGizmoTransform(GizmoTransform);
	}

	if (TRSGizmo)
	{
		UInteractiveToolManager* ToolManager = GetToolManager();
		const IToolsContextQueriesAPI* ToolsContextQueries = ToolManager ? ToolManager->GetContextQueriesAPI() : nullptr;
		const bool bWorld = ToolsContextQueries ? ToolsContextQueries->GetCurrentCoordinateSystem() == EToolContextCoordinateSystem::World : false;

		// note: this is a trick to avoid cycling notifications... (cf. UCombinedTransformGizmo::ReinitializeGizmoTransform)
		TGuardValue<bool> PivotModeGuard(TransformProxy->bSetPivotMode, true);
		if (bWorld)
		{
			const FTransform TransformNoRot(FQuat::Identity, GizmoTransform.GetTranslation(), GizmoTransform.GetScale3D());
			TransformProxy->SetTransform(TransformNoRot);			
		}
		else
		{
			TransformProxy->SetTransform(GizmoTransform);
		}
	}

}

FInputRayHit USequencerPivotTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	//OnUpdateModifierState is not called yet so use modifier keys unfortunately
	FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
	bPickingPivotLocation = KeyState.IsControlDown();
	if (bPickingPivotLocation == true)
	{
		FVector Temp;
		FInputRayHit Result = FindRayHit(ClickPos.WorldRay, Temp);
		return Result;
	}
	return FInputRayHit();
}

void USequencerPivotTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (bPickingPivotLocation)
	{
		FVector HitLocation;
		FInputRayHit Result = FindRayHit(ClickPos.WorldRay, HitLocation);
		if (Result.bHit)
		{
			GizmoTransform.SetLocation(HitLocation);
			UpdateGizmoTransform();
			SavePivotTransforms();
		}
	}
}

FInputRayHit USequencerPivotTool::FindRayHit(const FRay& WorldRay, FVector& HitPos)
{
	// trace a ray into the World
	if (TargetWorld)
	{
		FCollisionObjectQueryParams QueryParams(FCollisionObjectQueryParams::AllObjects);
		FHitResult Result;
		bool bHitWorld = TargetWorld->LineTraceSingleByObjectType(Result, WorldRay.Origin, WorldRay.PointAt(999999), QueryParams);
		if (bHitWorld)
		{
			HitPos = Result.ImpactPoint;
			return FInputRayHit(Result.Distance);
		}
	}
	return FInputRayHit();
}



void USequencerPivotTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	// if the user updated any of the property fields, update the distance
	//UpdateDistance();
}


#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerAnimEditPivotTool)


void USequencerPivotTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (RenderAPI == nullptr)
	{
		return;
	}
	if (bPickingPivotLocation == false )
	{
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

		const bool bHitTesting = PDI && PDI->IsHitTesting();
		float KeySize = 20.0f;
		//hack for PDI->DrawPoint not scaling perspective keys see FViewElementPDI::DrawPoint
		if (RenderAPI->GetSceneView())
		{
			const bool bIsPerspective = (RenderAPI->GetSceneView()->ViewMatrices.GetViewToClip().M[3][3] < 1.0f) ? true : false;
			if (bIsPerspective)
			{
				const float ZoomFactor = FMath::Min<float>(RenderAPI->GetSceneView()->ViewMatrices.GetViewToClip().M[0][0], RenderAPI->GetSceneView()->ViewMatrices.GetViewToClip().M[1][1]);
				KeySize = KeySize / ZoomFactor;
			}
		}
		const FLinearColor Color = FLinearColor(1.0f, 0.0f, 0.0f);
		if (bHitTesting)
		{
			PDI->SetHitProxy(new HSequencerPivotProxy());
		}

		PDI->DrawPoint(GizmoTransform.GetLocation(), Color, KeySize, SDPG_MAX);

		if (bHitTesting)
		{
			PDI->SetHitProxy(nullptr);
		}
	}
}

//Pivot Overlay functions

FVector2f USequencerPivotTool::LastPivotOverlayLocation = FVector2f::ZeroVector;

class SPivotOverlayWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPivotOverlayWidget) {}
	SLATE_ARGUMENT(USequencerPivotTool*, InPivotTool)
	SLATE_END_ARGS()
	~SPivotOverlayWidget()
	{
	}

	void Construct(const FArguments& InArgs)
	{
		OwningPivotTool = InArgs._InPivotTool;
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
			.Padding(10.f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(0.0f, 0.0f, 0.0f, 2.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "NormalText.Important")
					.Text(LOCTEXT("PivotTool", "Pivot Tool"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(2)
					+ SUniformGridPanel::Slot(0, 0)
					[	
						SNew(SButton)
						.OnClicked_Raw(this, &SPivotOverlayWidget::OnButtonClicked)
						.Text(LOCTEXT("Edit", "Edit"))
						.ToolTipText(LOCTEXT("Edit_Tooltip","When in edit mode you can move the pivot location freely without changing the object's transform"))
						.ButtonColorAndOpacity(this, &SPivotOverlayWidget::GetEditColor)				
					]
					+ SUniformGridPanel::Slot(1, 0)
					[							
						SNew(SButton)
						.OnClicked_Raw(this, &SPivotOverlayWidget::OnButtonClicked)
						.Text(LOCTEXT("Pose", "Pose"))
						.ToolTipText(LOCTEXT("Pose_Tooltip", "In pose mode you will rotate or translate the object about its pivot"))
						.ButtonColorAndOpacity(this, &SPivotOverlayWidget::GetPoseColor)
					]
				]
			]
		];
	}

private:
	TWeakObjectPtr<USequencerPivotTool> OwningPivotTool;

	FReply OnButtonClicked()
	{
		if (OwningPivotTool.IsValid())
		{
			OwningPivotTool.Get()->TogglePivotMode();
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	FSlateColor GetEditColor() const
	{
		if (OwningPivotTool.IsValid())
		{
			return OwningPivotTool.Get()->IsInPivotMode() ? FStyleColors::Transparent : FStyleColors::Select;
		}
		return FStyleColors::Transparent;
	}

	FSlateColor GetPoseColor() const
	{
		if (OwningPivotTool.IsValid())
		{
			return OwningPivotTool.Get()->IsInPivotMode() ? FStyleColors::Select : FStyleColors::Transparent;
		}
		return FStyleColors::Transparent;
	}

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		// Need to remember where within a tab we grabbed
		const FVector2D TabGrabScreenSpaceOffset = MouseEvent.GetScreenSpacePosition() - MyGeometry.GetAbsolutePosition();

		FOnInViewportUIDropped OnUIDropped = FOnInViewportUIDropped::CreateRaw(this, &SPivotOverlayWidget::FinishDraggingWidget);
		// Start dragging.
		TSharedRef<FInViewportUIDragOperation> DragDropOperation =
			FInViewportUIDragOperation::New(
				SharedThis(this),
				TabGrabScreenSpaceOffset,
				GetDesiredSize(),
				OnUIDropped
			);

		if (OwningPivotTool.IsValid())
		{
			OwningPivotTool.Get()->TryRemovePivotOverlay();
		}
		return FReply::Handled().BeginDragDrop(DragDropOperation);
	}

	void FinishDraggingWidget(const FVector2D InLocation)
	{
		if (OwningPivotTool.IsValid())
		{
			if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
			{
				TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule->GetFirstActiveViewport();
				if (ActiveLevelViewport.IsValid())
				{
					OwningPivotTool.Get()->UpdatePivotOverlayLocation(InLocation, ActiveLevelViewport);
					OwningPivotTool.Get()->TryShowPivotOverlay();
				}
			}
		}
	}
};

static FVector2f GetActiveViewportSize(TSharedPtr<IAssetViewport>& ActiveViewport)
{
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		ActiveViewport->GetActiveViewport(),
		ActiveViewport->GetAssetViewportClient().GetScene(),
		ActiveViewport->GetAssetViewportClient().EngineShowFlags)
		.SetRealtimeUpdate(ActiveViewport->GetAssetViewportClient().IsRealtime()));
	// SceneView is deleted with the ViewFamily
	FSceneView* SceneView = ActiveViewport->GetAssetViewportClient().CalcSceneView(&ViewFamily);
	const float InvDpiScale = 1.0f / ActiveViewport->GetAssetViewportClient().GetDPIScale();
	const float MaxX = SceneView->UnscaledViewRect.Width() * InvDpiScale;
	const float MaxY = SceneView->UnscaledViewRect.Height() * InvDpiScale;
	return FVector2f(MaxX, MaxY);
}

void USequencerPivotTool::CreateAndShowPivotOverlay()
{
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
	{
		TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule->GetFirstActiveViewport();
		if (ActiveLevelViewport.IsValid())
		{
			FVector2f NewWidgetLocation = LastPivotOverlayLocation;

			if (NewWidgetLocation.IsZero())
			{
				const FVector2f ActiveViewportSize = GetActiveViewportSize(ActiveLevelViewport);
				NewWidgetLocation.X = ActiveViewportSize.X / 2.0f;
				NewWidgetLocation.Y = 50.0f;			

			}
			UpdatePivotOverlayLocation(FVector2D(NewWidgetLocation), ActiveLevelViewport);

			TAttribute<FMargin> MarginPadding = TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateUObject(this, &USequencerPivotTool::GetPivotOverlayPadding));

			SAssignNew(PivotWidget, SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Left)
				.Padding(MarginPadding)
				[
					SNew(SPivotOverlayWidget)
					.InPivotTool(this)
				];

			if (PivotWidget)
			{
				ActiveLevelViewport->AddOverlayWidget(PivotWidget.ToSharedRef());
			}
		}
	}
}

FMargin USequencerPivotTool::GetPivotOverlayPadding() const
{
	return FMargin(LastPivotOverlayLocation.X, LastPivotOverlayLocation.Y, 0, 0);
}

void USequencerPivotTool::RemoveAndDestroyPivotOverlay()
{
	if (PivotWidget.IsValid() == false)
	{
		return;
	}
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
	{
		if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetFirstLevelEditor())
		{
			for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
			{
				if (LevelViewport.IsValid())
				{
					if (PivotWidget)
					{
						LevelViewport->RemoveOverlayWidget(PivotWidget.ToSharedRef());
					}
				}
			}
			PivotWidget.Reset();
		}
	}
}

void USequencerPivotTool::TryRemovePivotOverlay()
{
	if (PivotWidget.IsValid() == false)
	{
		return;
	}
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
	{
		if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetFirstLevelEditor())
		{
			for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
			{
				if (LevelViewport.IsValid())
				{
					if (PivotWidget)
					{
						LevelViewport->RemoveOverlayWidget(PivotWidget.ToSharedRef());
					}
				}
			}
		}
	}
}

void USequencerPivotTool::TryShowPivotOverlay()
{
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
	{
		TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule->GetFirstActiveViewport();
		if (ActiveLevelViewport.IsValid())
		{
			if (PivotWidget)
			{
				ActiveLevelViewport->AddOverlayWidget(PivotWidget.ToSharedRef());
			}
		}
	}
}

void USequencerPivotTool::UpdatePivotOverlayLocation(const FVector2D InLocation, TSharedPtr<IAssetViewport> ActiveLevelViewport)
{
	const FVector2f ActiveViewportSize = GetActiveViewportSize(ActiveLevelViewport);
	FVector2f ScreenPos = UE::Slate::CastToVector2f(InLocation);

	const float EdgeFactor = 0.97f;
	const float MinX = ActiveViewportSize.X * (1 - EdgeFactor);
	const float MinY = ActiveViewportSize.Y * (1 - EdgeFactor);
	const float MaxX = ActiveViewportSize.X * EdgeFactor;
	const float MaxY = ActiveViewportSize.Y * EdgeFactor;
	const bool bOutside = ScreenPos.X < MinX || ScreenPos.X > MaxX || ScreenPos.Y < MinY || ScreenPos.Y > MaxY;
	if (bOutside)
	{
		// reset the location if it was placed out of bounds
		ScreenPos.X = ActiveViewportSize.X / 2.0f;
		ScreenPos.Y = 50.0f;
	}
	LastPivotOverlayLocation = ScreenPos;
}

void USequencerPivotTool::PrepareRigForManipulation(UControlRig* InControlRig)
{
	using namespace UE::AIE::PivotTool;
	
	URigHierarchy* Hierarchy = InControlRig ? InControlRig->GetHierarchy() : nullptr;
	if (!Hierarchy)
	{
		return;
	}
		
	const TArray<FName> SelectedControls = CurrentControlSelection(InControlRig);
	if (SelectedControls.IsEmpty())
	{
		return;
	}

	// store controls to transform
	TArray<FRigControlElement*> Controls;
	Controls.Reserve(SelectedControls.Num());
	for (const FName& Name : SelectedControls)
	{
		if (FRigControlElement* ControlElement = InControlRig->FindControl(Name))
		{
			//if we have a parent that's selected and we are a float/bool/int/enum skip
			if (FRigControlElement* ParentControlElement = Cast<FRigControlElement>(Hierarchy->GetFirstParent(ControlElement)))
			{
				if (InControlRig->IsControlSelected(ParentControlElement->GetFName()) && (
					ControlElement->Settings.ControlType == ERigControlType::Bool || 
					ControlElement->Settings.ControlType == ERigControlType::Float || 
					ControlElement->Settings.ControlType == ERigControlType::ScaleFloat || 
					ControlElement->Settings.ControlType == ERigControlType::Integer)
					)
				{
					continue;
				}
			}
			Controls.Add(ControlElement);
		}
	}
			
	if (Controls.IsEmpty())
	{
		return;
	}
			
	Hierarchy->Modify();
	
	// keep last selected control if bShiftPressedWhenStarted
	FRigControlElement* LastControl = nullptr;
	if (bShiftPressedWhenStarted && !Controls.IsEmpty())
	{
		LastControl = Controls.Last();
		Controls.RemoveAt(Controls.Num() - 1);
	}

	// sort control from depth to avoid setting a child before it's parent
	Algo::Sort(Controls, [Hierarchy](const FRigControlElement* Control0, const FRigControlElement* Control1)
	{
		constexpr bool bRecursive = true;	
		const FRigBaseElementParentArray& Parents = Hierarchy->GetParents(Control0, bRecursive);
		const FRigBaseElementParentArray& OtherParents = Hierarchy->GetParents(Control1, bRecursive);

		int32 Index = Parents.Num(), OtherIndex = OtherParents.Num();
		auto HasCommonAncestor = [&Index, &Parents, &OtherIndex, &OtherParents, Control1, Control0]()
		{
			if (!Parents.IsEmpty() && !OtherParents.IsEmpty())
			{
				FRigBaseElement* CommonAncestor = nullptr;
				while (Index > 0 && OtherIndex > 0 && Parents[Index-1] == OtherParents[OtherIndex-1])
				{
					CommonAncestor = Parents[Index-1];
					--Index;
					--OtherIndex;
				}
				return CommonAncestor != nullptr;
			}
			return false;
		};

		// if common ancestor, compare depths
		if (HasCommonAncestor() && Index != OtherIndex)
		{
			return OtherIndex > Index; 
		}

		// otherwise, compare indices
		if (Control1->GetIndex() > Control0->GetIndex())
		{
			return true;
		}

		return false;
	});
	
	// re-add last selected control stored if bShiftPressedWhenStarted
	if (LastControl)
	{
		Controls.Add(LastControl);
	}
			
	UE::ControlRigEditMode::FInteractionDependencyCache& NewDependency = InteractionDependencies.Add(InControlRig);
	NewDependency.WeakHierarchy = Hierarchy;
	
	const FRigDependenciesProviderForControlRig DependencyProvider(InControlRig);
	for (int32 Index = 0; Index < Controls.Num(); ++Index)
	{
		FRigControlElement* Control = Controls[Index];
		
		FControlRigSelectionDuringDrag ControlDrag;
		ControlDrag.LevelSequence = LevelSequence;
		ControlDrag.ControlName = Control->GetFName();
		ControlDrag.ControlRig = InControlRig;
		ControlDrag.CurrentTransform = GetControlWorldSpaceTransform(ControlDrag.ControlRig, ControlDrag.ControlName);
		ControlRigDrags.Add(ControlDrag);
				
		for (int32 NextIndex = Index+1; NextIndex < Controls.Num(); ++NextIndex)
		{
			FRigControlElement* OtherControl = Controls[NextIndex];
			if (Hierarchy->IsParentedTo(Control, OtherControl, DependencyProvider))
			{
				NewDependency.Parents.Add(OtherControl->GetKey());
				NewDependency.Children.Add(Control->GetKey());
			}
			else if (Hierarchy->IsParentedTo(OtherControl, Control, DependencyProvider))
			{
				NewDependency.Parents.Add(Control->GetKey());
				NewDependency.Children.Add(OtherControl->GetKey());
			}
		}

		// store parents' pose versions
		const FRigBaseElementParentArray Parents = Hierarchy->GetParents(Control);
		NewDependency.ParentsPoseVersion.Reserve(Parents.Num());
		for (const FRigBaseElement* Parent: Parents)
		{
			if (const FRigTransformElement* TransformParent = Cast<FRigTransformElement>(Parent))
			{
				NewDependency.ParentsPoseVersion.FindOrAdd(Parent->GetIndex()) = Hierarchy->GetPoseVersion(TransformParent);
			}
		}
	}
}

void USequencerPivotTool::PrepareActorForManipulation(AActor* InActor)
{
	if (!InActor)
	{
		return;
	}
	
	if (USceneComponent* RootComponent = InActor->GetRootComponent())
	{
		if (RootComponent->HasAnyFlags(RF_DefaultSubObject))
		{
			// Default subobjects must be included in any undo/redo operations
			RootComponent->SetFlags(RF_Transactional);
		}
		RootComponent->Modify();
	}

	FActorSelectonDuringDrag ActorDrag;
	ActorDrag.LevelSequence = LevelSequence;
	ActorDrag.Actor = InActor;
	ActorDrag.CurrentTransform = InActor->GetTransform();
	ActorDrags.Add(ActorDrag);
}

void USequencerPivotTool::UpdateBoundComponents()
{
	if (!bManipulatorMadeChange)
	{
		return;
	}
	
	for (const TSharedPtr<FControlRigInteractionScope>& InteractionScope: InteractionScopes)
	{
		if (UControlRig* ControlRig = InteractionScope->GetControlRig())
		{
			USceneComponent* SceneComponent = UE::AIE::PivotTool::GetBoundComponent(ControlRig);
			if (UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(SceneComponent))
			{
				ControlRigComponent->Update();
			}
			else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(SceneComponent))
			{
				if (!ControlRig->ContainsSimulation())
				{
					constexpr float DeltaTime = 0.f;
					constexpr bool bIncludeFromChildActors = true, bDoNotNeedValidRootMotion = false;
					const AActor* Parent = SkeletalMeshComponent->GetOwner();
					while (Parent)
					{
						Parent->ForEachComponent<USkeletalMeshComponent>(bIncludeFromChildActors, [](USkeletalMeshComponent* SkeletalMeshToUpdate)
						{
							if (SkeletalMeshToUpdate)
							{
								SkeletalMeshToUpdate->TickAnimation(DeltaTime, bDoNotNeedValidRootMotion);
								SkeletalMeshToUpdate->RefreshBoneTransforms();
								SkeletalMeshToUpdate->RefreshFollowerComponents	();
								SkeletalMeshToUpdate->UpdateComponentToWorld();
								SkeletalMeshToUpdate->FinalizeBoneTransform();
								SkeletalMeshToUpdate->MarkRenderTransformDirty();
								SkeletalMeshToUpdate->MarkRenderDynamicDataDirty();
							}								
						});
						Parent = Parent->GetAttachParentActor();
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE


