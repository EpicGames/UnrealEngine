// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerAnimToolsModule.h"
#include "SequencerAnimEditPivotTool.h"

#include "EditorModeManager.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "InteractiveToolManager.h"
#include "LevelEditorSequencerIntegration.h"

#include "Modules/ModuleManager.h"
#include "MotionTrailTool.h"
#include "Tools/MotionTrailOptions.h"

#define LOCTEXT_NAMESPACE "FSequencerAnimToolsModule"

namespace UE
{
namespace SequencerAnimTools
{

void FSequencerAnimToolsModule::StartupModule()
{
	UMotionTrailToolOptions* MotionTrailOptions = GetMutableDefault<UMotionTrailToolOptions>();
	if (MotionTrailOptions)
	{
		MotionTrailOptions->OnDisplayPropertyChanged.AddRaw(this, &FSequencerAnimToolsModule::OnMotionTralOptionChanged);
	}
	FEditPivotCommands::Register();
	FMotionTrailCommands::Register();

}

void FSequencerAnimToolsModule::ShutdownModule()
{
	/* can't do this for some reason the object is already dead (but not nullptr so can't check)
	UMotionTrailToolOptions* MotionTrailOptions = GetMutableDefault<UMotionTrailToolOptions>();
	if (MotionTrailOptions)
	{
		MotionTrailOptions->OnDisplayPropertyChanged.RemoveAll(this);
	}
	*/
	FEditPivotCommands::Unregister();
	FMotionTrailCommands::Unregister();

}

void FSequencerAnimToolsModule::OnMotionTralOptionChanged(FName PropertyName)
{
	FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
	UMotionTrailToolOptions* MotionTrailOptions = GetMutableDefault<UMotionTrailToolOptions>();
	if (MotionTrailOptions && LevelEditorModule)
	{
		TWeakPtr<ILevelEditor> LevelEditorPtr = LevelEditorModule->GetLevelEditorInstance();
		if (PropertyName == FName("bShowTrails") && LevelEditorPtr.IsValid())
		{
			static bool bIsChangingTrail = false;

			if (bIsChangingTrail == false)
			{
				bIsChangingTrail = true;
				if (FLevelEditorSequencerIntegration::Get().GetSequencers().Num() > 0)
				{
					if (MotionTrailOptions->bShowTrails)
					{
						LevelEditorPtr.Pin()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->SelectActiveToolType(EToolSide::Left, TEXT("SequencerMotionTrail"));
						LevelEditorPtr.Pin()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->ActivateTool(EToolSide::Left);
					}
					else
					{
						if (Cast<UMotionTrailTool>(LevelEditorPtr.Pin()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->ActiveLeftTool))
						{
							LevelEditorPtr.Pin()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->DeactivateTool(EToolSide::Left, EToolShutdownType::Completed);
						}
					}
					
				}
				else
				{
					MotionTrailOptions->bShowTrails = false;
				}
				bIsChangingTrail = false;
			}
		}
	}
}

} // namespace SequencerAnimTools
} // namespace UE

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(UE::SequencerAnimTools::FSequencerAnimToolsModule, SequencerAnimTools)
