// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerToolsEditMode.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "EditorModeManager.h"
#include "InteractiveToolManager.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "MotionTrailTool.h"
#include "SequencerAnimEditPivotTool.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerToolsEditMode)

#define LOCTEXT_NAMESPACE "SequencerAnimTools"


FEditorModeID USequencerToolsEditMode::ModeName = TEXT("SequencerToolsEditMode");


USequencerToolsEditMode::USequencerToolsEditMode()
	: bDeactivateOnPIEStartStateToRestore(false)
	, bDeactivateOnSaveWorldToRestore(false)
{

	Info = FEditorModeInfo(
		ModeName,
		LOCTEXT("ModeName", "Sequencer Tools"),
		FSlateIcon(),
		false
	);
}

USequencerToolsEditMode::~USequencerToolsEditMode()
{

}

void USequencerToolsEditMode::Enter()
{
	Super::Enter();

	Owner->OnEditorModeIDChanged().AddUObject(this, &USequencerToolsEditMode::LocalOnModeActivated);

	FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
	
	if (LevelEditorModule)
	{
		TWeakPtr<ILevelEditor> LevelEditorPtr = LevelEditorModule->GetLevelEditorInstance();
		
		if (LevelEditorPtr.IsValid())
		{
			LevelEditorPtr.Pin()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->RegisterToolType(TEXT("SequencerMotionTrail"), NewObject<UMotionTrailToolBuilder>(this));
			LevelEditorPtr.Pin()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->RegisterToolType(TEXT("SequencerPivotTool"), NewObject<USequencerPivotToolBuilder>(this));
			LevelEditorPtr.Pin()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->ConfigureChangeTrackingMode(EToolChangeTrackingMode::FullUndoRedo);
			// Currently there are some issues when the gizmo context object is shared among modes, so make sure to
			// register it in the mode-level context store so it's not picked up by other modes.
			UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(GetInteractiveToolsContext(EToolsContextScope::EdMode));
		}
	}
}

void USequencerToolsEditMode::Exit()
{
	FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
	
	if (LevelEditorModule)
	{
		TWeakPtr<ILevelEditor> LevelEditorPtr = LevelEditorModule->GetLevelEditorInstance();
		
		if (LevelEditorPtr.IsValid())
		{
			LevelEditorPtr.Pin()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->UnregisterToolType(TEXT("SequencerMotionTrail"));
			LevelEditorPtr.Pin()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->UnregisterToolType(TEXT("SequencerPivotTool"));

			UE::TransformGizmoUtil::DeregisterTransformGizmoContextObject(GetInteractiveToolsContext(EToolsContextScope::EdMode));
		}
	}
	Owner->OnEditorModeIDChanged().RemoveAll(this);

	Super::Exit();
}

void USequencerToolsEditMode::LocalOnModeActivated(const FEditorModeID& InID, bool bIsActive)
{
	if (InID == GetID())
	{
		UEditorInteractiveToolsContext* EditorInteractiveToolsContext = GetInteractiveToolsContext(EToolsContextScope::Editor);
		if (bIsActive)
		{
			bDeactivateOnPIEStartStateToRestore = EditorInteractiveToolsContext->GetDeactivateToolsOnPIEStart();
			EditorInteractiveToolsContext->SetDeactivateToolsOnPIEStart(false);

			bDeactivateOnSaveWorldToRestore = EditorInteractiveToolsContext->GetDeactivateToolsOnSaveWorld();
			EditorInteractiveToolsContext->SetDeactivateToolsOnSaveWorld(false);
		}
		else
		{
			EditorInteractiveToolsContext->SetDeactivateToolsOnPIEStart(bDeactivateOnPIEStartStateToRestore);
			EditorInteractiveToolsContext->SetDeactivateToolsOnSaveWorld(bDeactivateOnSaveWorldToRestore);
		}
	}
}

bool USequencerToolsEditMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	// Compatible with all modes similar to FSequencerEdMode
	return true;
}

//If we have one of our own active tools we pass input to it's commands.
bool USequencerToolsEditMode::InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) 
{
	if (InEvent != IE_Released)
	{
		//	MZ doesn't seem needed, need more testing  TGuardValue<FEditorViewportClient*> ViewportGuard(CurrentViewportClient, InViewportClient);
		if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
		{
			TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin();

			if (LevelEditor.IsValid())
			{
				if (IBaseSequencerAnimTool* Tool = Cast<IBaseSequencerAnimTool>(LevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->GetActiveTool(EToolSide::Left)))
				{
					if (Tool->ProcessCommandBindings(InKey, (InEvent == IE_Repeat)))
					{
						return true;
					}
				}
			}
		}
	}	
	return UBaseLegacyWidgetEdMode::InputKey(InViewportClient, InViewport, InKey, InEvent);
}

#undef LOCTEXT_NAMESPACE
