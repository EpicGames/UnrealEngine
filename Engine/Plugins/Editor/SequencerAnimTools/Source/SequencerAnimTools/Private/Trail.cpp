// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trail.h"

//MZ TODO LOOK AT REIMPLEMENTING CONSTANT TRAILS
namespace UE
{
namespace SequencerAnimTools
{
bool FTrail::HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap)
{
	if (UObject*const* NewObject = ReplacementMap.Find(Owner.Get()))
	{
		Owner = *NewObject;
		return true;
	}
	return false;
}
}
}