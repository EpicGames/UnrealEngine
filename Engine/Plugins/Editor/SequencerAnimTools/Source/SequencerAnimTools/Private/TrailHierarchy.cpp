// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrailHierarchy.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "Layout/WidgetPath.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "SceneManagement.h"
#include "Tools/MotionTrailOptions.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "CanvasTypes.h"
#include "Sequencer/SequencerTrailHierarchy.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "ScopedTransaction.h"


#define LOCTEXT_NAMESPACE "MotionTrailEditorMode"

namespace UE
{
namespace SequencerAnimTools
{
using namespace UE::AIE;


IMPLEMENT_HIT_PROXY(HBaseTrailProxy, HHitProxy);
IMPLEMENT_HIT_PROXY(HNewMotionTrailProxy, HBaseTrailProxy);


void FTrail::GetColor(const FFrameNumber& CurrentTime, FColorState& State)
{
	if (State.GetStyle() == EMotionTrailTrailStyle::Time)
	{
		if (CurrentTime < State.SequencerTime)
		{
			State.CalculatedColor = State.Options->TimePreColor;
		}
		else
		{
			State.CalculatedColor = State.Options->TimePostColor;
		}
	}
	else if (State.GetStyle() == EMotionTrailTrailStyle::Dashed)
	{

		if (State.bFirstFrame == true)
		{
			State.bFirstFrame = false;
		}

		const int32 Index = (CurrentTime.Value - State.StartFrame.Value) / State.TicksPerFrame.Value;

		if (Index % 2 == 0)
		{
			State.CalculatedColor = State.Options->DashPreColor;
		}
		else
		{
			State.CalculatedColor = State.Options->DashPostColor;
		}
	}
	else
	{
		State.CalculatedColor = State.Options->DefaultColor;
		
	}
}

void FTrailHierarchyRenderer::Render(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	
	if (CachedOptions)
	{
		//options object won't change over lifetime of the renderer functions
		const FDateTime RenderStartTime = FDateTime::Now();

		const int32 NumEvalTimes = int32((OwningHierarchy->GetViewFrameRange().GetUpperBoundValue().Value - OwningHierarchy->GetViewFrameRange().GetLowerBoundValue().Value) / OwningHierarchy->GetFramesPerFrame().Value) + 1;
		if (NumEvalTimes <= 1)
		{
			return;
		}
		int32 NumLinesReserveSize = int32(NumEvalTimes * OwningHierarchy->GetAllTrails().Num() * 1.3);
		PDI->AddReserveLines(SDPG_Foreground, NumLinesReserveSize);

		FColorState ColorState;
		ColorState.Setup(OwningHierarchy);
		const FCurrentFramesInfo* CurrentFramesInfo = OwningHierarchy->GetCurrentFramesInfo();
		for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : OwningHierarchy->GetAllTrails())
		{
			FTrajectoryDrawInfo* CurDrawInfo = GuidTrailPair.Value->GetDrawInfo();
			if (CurDrawInfo && OwningHierarchy->GetVisibilityManager().IsTrailVisible(GuidTrailPair.Key, GuidTrailPair.Value.Get(), CachedOptions->bShowSelectedTrails))
			{
				if (OwningHierarchy->IsTrailEvaluating(GuidTrailPair.Key, true))
				{
					GuidTrailPair.Value->RenderEvaluating(GuidTrailPair.Key, View, PDI);
					continue;
				}
				if (GuidTrailPair.Value.Get()->GetCacheState() != ETrailCacheState::UpToDate)
				{
					continue;
				}
				//the above check so that if we are evaluating something that is getting drawn indirectly we skip drawing it, but if it is
				//getting manipulated we still want to draw it. If it's not evaluating though we don't use sparse time values, instead do every frame.
				const bool bTrailDirectlyEvaluating = OwningHierarchy->IsTrailEvaluating(GuidTrailPair.Key, false);
				GuidTrailPair.Value->ReadyToDrawTrail(ColorState, CurrentFramesInfo, bTrailDirectlyEvaluating, OwningHierarchy->GetVisibilityManager().IsTrailAlwaysVisible(GuidTrailPair.Key));
				GuidTrailPair.Value->Render(GuidTrailPair.Key, View, PDI, bTrailDirectlyEvaluating);
			}
			
		}
		const FTimespan RenderTimespan = FDateTime::Now() - RenderStartTime;
		OwningHierarchy->GetTimingStats().Add("FTrailHierarchyRenderer::Render", RenderTimespan);
	}
}

void FTrailHierarchyRenderer::DrawHUD(const FSceneView* View, FCanvas* Canvas)
{
	
	if (CachedOptions && CachedOptions->bShowMarks)
	{
		const FDateTime DrawHUDStartTime = FDateTime::Now();
		const FFrameNumber FramesPerMark = OwningHierarchy->GetFramesPerFrame();

		const int32 NumMarksPerTrail = int32((OwningHierarchy->GetViewFrameRange().GetUpperBoundValue().Value - OwningHierarchy->GetViewFrameRange().GetLowerBoundValue().Value) / FramesPerMark.Value) + 1;
		if (NumMarksPerTrail <= 1)
		{
			return;
		}
		const int32 PredictedNumMarks = int32(NumMarksPerTrail * OwningHierarchy->GetAllTrails().Num() * 1.3);

		Canvas->GetBatchedElements(FCanvas::EElementType::ET_Line)->AddReserveLines(PredictedNumMarks);
		FTrailScreenSpaceTransform Transform(View, Canvas->GetDPIScale());
		const FCurrentFramesInfo* CurrentFramesInfo = OwningHierarchy->GetCurrentFramesInfo();

		for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : OwningHierarchy->GetAllTrails())
		{
			if (OwningHierarchy->GetVisibilityManager().IsTrailVisible(GuidTrailPair.Key, GuidTrailPair.Value.Get(), CachedOptions->bShowSelectedTrails))
			{
				if (OwningHierarchy->IsTrailEvaluating(GuidTrailPair.Key, false) || GuidTrailPair.Value.Get()->GetCacheState() != ETrailCacheState::UpToDate)
				{
					continue;
				}
				TArray<FVector2D> Marks, MarkNormals;
				GuidTrailPair.Value->GetTickPointsForDisplay(Transform, *CurrentFramesInfo, false, Marks, MarkNormals);
				if (Marks.Num() > 0)
				{
					FColorState ColorState;
					ColorState.Setup(OwningHierarchy);
					const FLinearColor Color = UMotionTrailToolOptions::GetTrailOptions()->MarkColor;
					const FLinearColor CurrentFrameMarkColor = UMotionTrailToolOptions::GetTrailOptions()->CurrentFrameMarkColor;
					for (int32 Idx = 0; Idx < Marks.Num(); Idx++)
					{
						const FFrameNumber Frame = CurrentFramesInfo->CurrentFrameTimes.CalculateFrame(Idx);

						const FVector2D StartPoint = Marks[Idx] - MarkNormals[Idx] * UMotionTrailToolOptions::GetTrailOptions()->MarkSize;
						const FVector2D EndPoint = Marks[Idx] + MarkNormals[Idx] * UMotionTrailToolOptions::GetTrailOptions()->MarkSize;
						FCanvasLineItem LineItem = FCanvasLineItem(StartPoint, EndPoint);
						if (ColorState.SequencerTime != Frame)
						{
							LineItem.SetColor(Color);
						}
						else
						{
							LineItem.SetColor(CurrentFrameMarkColor);
						}
						Canvas->DrawItem(LineItem);
					}
				}
			}
			GuidTrailPair.Value->DrawHUD(View, Canvas);

		}

		const FTimespan DrawHUDTimespan = FDateTime::Now() - DrawHUDStartTime;
		OwningHierarchy->GetTimingStats().Add("FTrailHierarchyRenderer::DrawHUD", DrawHUDTimespan);
	}
}

class STrailOptionsPopup : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STrailOptionsPopup)
	{}

	SLATE_ARGUMENT(FTrailVisibilityManager*, InVisibilityManager)
	SLATE_ARGUMENT(FGuid, InTrailGuid)

	SLATE_END_ARGS()


public:
	FTrailVisibilityManager* VisibilityManager;
	FGuid TrailGuid;

	void SetPinned(ECheckBoxState NewState)
	{
		UMotionTrailToolOptions* Settings = UMotionTrailToolOptions::GetTrailOptions();
		if (NewState == ECheckBoxState::Checked)
		{
			Settings->PinSelection();
		}
		else
		{
			if (Settings->GetNumPinned() > 0)
			{
				for (int32 Index = 0; Index < Settings->GetNumPinned(); ++Index)
				{
					if (UMotionTrailToolOptions::FPinnedTrail* Trail = Settings->GetPinnedTrail(Index))
					{
						if (Trail->TrailGuid == TrailGuid)
						{
							Settings->DeletePinned(Index);
							break;
						}
					}
				}
			}
		}
	}
	ECheckBoxState IsTrailAlwaysVisible() const
	{
		if (VisibilityManager)
		{
			return VisibilityManager->IsTrailAlwaysVisible(TrailGuid) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		return ECheckBoxState::Undetermined;
	}

	FReply ShowOptions()
	{
		const FName ControlRigMotionTrailTab("ControlRigMotionTrailTab");
		FGlobalTabmanager::Get()->TryInvokeTab(ControlRigMotionTrailTab);
		return FReply::Handled();
	}

	void Construct(const FArguments& InArgs)
	{
		VisibilityManager = InArgs._InVisibilityManager;
		TrailGuid = InArgs._InTrailGuid;
		// Then make widget
		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Background")))
			.Padding(5)
			.Content()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 1.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.OnCheckStateChanged(this, &STrailOptionsPopup::SetPinned)
						.IsChecked(this, &STrailOptionsPopup::IsTrailAlwaysVisible)
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Text(LOCTEXT("Pinned", "Pinned"))
					]	
				]
			]
		];
		
	}
};

void FTrailHierarchy::OpenContextMenu(const FGuid& TrailGuid)
{
	/* Turned off for now 
	* 
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

	TSharedPtr<SWidget> MenuWidget;
	if(const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(TrailGuid))
	{
		MenuWidget =
			SNew(STrailOptionsPopup)
			.InVisibilityManager(&GetVisibilityManager())
			.InTrailGuid(TrailGuid);

		// Create as context menu
		FSlateApplication::Get().PushMenu(
			LevelEditor.ToSharedRef(),
			FWidgetPath(),
			MenuWidget.ToSharedRef(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	}
	*/
}

bool FTrailHierarchy::IsHitByClick(HHitProxy* InHitProxy)
{
	if (InHitProxy)
	{
		if (HBaseTrailProxy* HitProxy = HitProxyCast<HBaseTrailProxy>(InHitProxy))
		{
			return true;
		}
	}
	
	return false;
}

bool FTrailHierarchy::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy,FInputClick Click)
{
	bool bClickIsHandled = false;
	
	if (HNewMotionTrailProxy* HitProxy = HitProxyCast<HNewMotionTrailProxy>(InHitProxy))
	{
		if (Click.bIsRightMouse)
		{
			OpenContextMenu(HitProxy->Guid);
			return true;
		}
	}
	

	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair :GetAllTrails())
	{
		bool bIsClickHandled = GuidTrailPair.Value->HandleClick(GuidTrailPair.Key, InViewportClient, InHitProxy, Click);
		if (bIsClickHandled)
		{
			bClickIsHandled = true;
		}
	}

	if (bClickIsHandled)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("HandleClick", "Handle Click"), !GIsTransacting);

		if (UMotionTrailToolOptions::GetTrailOptions()->bShowSelectedTrails == false && Click.bShiftIsDown == false)
		{
			//use control rig edit mode to clear selction, since it handles everything
			if (FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName)))
			{
				ControlRigEditMode->ClearSelection();
			}
		}
	}
	

	return bClickIsHandled;
}

bool FTrailHierarchy::IsAnythingSelected() const
{
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		bool bHandled = GuidTrailPair.Value->IsAnythingSelected();
		if (bHandled)
		{
			return true;
		}
	}
	return false;
}

bool FTrailHierarchy::IsAnythingSelected(FVector& OutVectorPosition, FQuat& OutRotation) const
{
	OutVectorPosition = FVector::ZeroVector;
	OutRotation = FQuat::Identity;
	int NumSelected = 0;
	TOptional<FQuat> NewRotation;
	FQuat LocalRotation;
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		FVector Location;
		bool bHandled = GuidTrailPair.Value->IsAnythingSelected(Location,LocalRotation);
		if (bHandled)
		{
			OutVectorPosition += Location; 
			if (NewRotation.IsSet() == false)
			{
				NewRotation = LocalRotation;
			}
			++NumSelected;
		}
	}

	if (NumSelected > 0)
	{
		OutVectorPosition /= (double)NumSelected;
		OutRotation = NewRotation.GetValue();

	}
	return (NumSelected > 0);

}

bool FTrailHierarchy::IsAnythingSelected(TArray<FVector>& OutVectorPositions, bool bAllPositions) const
{
	OutVectorPositions.SetNum(0);
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		if (bAllPositions == false)
		{
			FVector Location;
			FQuat Rotation;
			bool bHandled = GuidTrailPair.Value->IsAnythingSelected(Location,Rotation);
			if (bHandled)
			{
				OutVectorPositions.Add(Location);
			}
		}
		else
		{
			GuidTrailPair.Value->IsAnythingSelected(OutVectorPositions);
		}
	}
	return (OutVectorPositions.Num() > 0);
}


bool FTrailHierarchy::IsSelected(const FGuid& Key) const
{
	const TUniquePtr<FTrail>* Trail = GetAllTrails().Find(Key);
	if (Trail != nullptr)
	{
		return (*Trail)->IsAnythingSelected();
	}
	return false;
}
void FTrailHierarchy::SelectNone()
{
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		GuidTrailPair.Value->SelectNone();
	}
}
void FTrailHierarchy::TranslateSelectedKeys(bool bRight)
{
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{		
		GuidTrailPair.Value->TranslateSelectedKeys(bRight);
	}
}

void FTrailHierarchy::DeleteSelectedKeys()
{
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		GuidTrailPair.Value->DeleteSelectedKeys();
	}
}

bool FTrailHierarchy::IsVisible(const FGuid& Key) const
{
	if (UMotionTrailToolOptions::GetTrailOptions()->bShowSelectedTrails == true)
	{
		return AllTrails.Contains(Key);
	}
	else return VisibilityManager.IsTrailAlwaysVisible(Key);
}

bool FTrailHierarchy::IsAlwaysVisible(const FGuid Key) const
{
	return VisibilityManager.IsTrailAlwaysVisible(Key);
}

bool FTrailHierarchy::BoxSelect(FBox& InBox, bool InSelect)
{
	bool bIsHandled = false;
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		bool bHandled = GuidTrailPair.Value->BoxSelect(InBox,InSelect);
		if (bHandled)
		{
			bIsHandled = true;
		}
	}
	return bIsHandled;
}
bool FTrailHierarchy::FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect)
{
	bool bIsHandled = false;
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		bool bHandled = GuidTrailPair.Value->FrustumSelect(InFrustum,InViewportClient,InSelect);
		if (bHandled)
		{
			bIsHandled = true;
		}
	}
	return bIsHandled;
}


bool FTrailHierarchy::StartTracking()
{
	bool bIsHandled = false;
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		bool bHandled = GuidTrailPair.Value->StartTracking();
		if (bHandled)
		{
			bIsHandled = true;
		}
	}
	return bIsHandled;
}

bool FTrailHierarchy::ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector &WidgetLocation, bool bApplyToOffset)
{
	bool bIsHandled = false;
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		bool bHandled =  GuidTrailPair.Value->ApplyDelta(Pos,Rot,WidgetLocation,bApplyToOffset);
		if (bHandled)
		{
			bIsHandled = true;
		}
	}
	return bIsHandled;
}

bool FTrailHierarchy::EndTracking()
{
	bool bIsHandled = false;
	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{
		bool bHandled = GuidTrailPair.Value->EndTracking();
		if (bHandled)
		{
			bIsHandled = true;
		}
	}
	return bIsHandled;
}


void FTrailHierarchy::CalculateEvalRangeArray()
{
	// new
	TicksPerSegment = GetFramesPerFrame();

	if (LastTicksPerSegment != TicksPerSegment || TickEvalRange != LastTickEvalRange)
	{
		for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
		{
			GuidTrailPair.Value->ForceEvaluateNextTick();
		}
		LastTicksPerSegment = TicksPerSegment;
		LastTickEvalRange = TickEvalRange;
	}
}
void FTrailHierarchy::Update()
{
	//new
	const FDateTime UpdateStartTime = FDateTime::Now();
	
	CalculateEvalRangeArray();
	VisibilityManager.InactiveMask.Reset();
	TArray<FGuid> DeadTrails;

	const bool bCheckForChange = CheckForChanges();

	for (const TPair<FGuid, TUniquePtr<FTrail>>& GuidTrailPair : GetAllTrails())
	{

		const FGuid CurGuid = GuidTrailPair.Key;
		FTrail::FNewSceneContext SceneContext = {
				bCheckForChange,
				CurGuid,
				this
		};
		if (!AllTrails.Contains(CurGuid))
		{
			continue;
		}
		//don't update if not visible
		if (IsVisible(GuidTrailPair.Key) == false)
		{
			continue;
		}
		// Update the trail
		FTrailCurrentStatus Status = AllTrails[CurGuid]->UpdateTrail(SceneContext);

		if (Status.CacheState == ETrailCacheState::Dead)
		{
			DeadTrails.Add(CurGuid);
		}
		if (Status.CacheState == ETrailCacheState::NotUpdated)
		{
			VisibilityManager.InactiveMask.Add(CurGuid);
		}
	}

	//remove dead trails
	for (const FGuid& TrailGuid : DeadTrails)
	{
		RemoveTrail(TrailGuid);
	}

	const FTimespan UpdateTimespan = FDateTime::Now() - UpdateStartTime;
	TimingStats.Add("FTrailHierarchy::Update", UpdateTimespan);

}

void FTrailHierarchy::AddTrail(const FGuid& Key, TUniquePtr<FTrail>&& TrailPtr)
{
	AllTrails.Add(Key, MoveTemp(TrailPtr));
}

void FTrailHierarchy::RemoveTrail(const FGuid& Key)
{
	AllTrails.Remove(Key);
}

void FTrailHierarchy::RemoveTrailIfNotAlwaysVisible(const FGuid& Key)
{
	if (IsAlwaysVisible(Key) == false)
	{
		RemoveTrail(Key);
	}
}

} // namespace SequencerAnimTools
} // namespace UE

#undef LOCTEXT_NAMESPACE
