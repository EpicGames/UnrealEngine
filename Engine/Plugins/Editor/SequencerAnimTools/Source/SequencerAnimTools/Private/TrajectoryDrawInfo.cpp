// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrajectoryDrawInfo.h"
#include "SceneView.h"
#include "Framework/Application/SlateApplication.h"


int32 UE::SequencerAnimTools::FFrameCalculator::IndicesToCalculatePerBucket = 60;
static FAutoConsoleVariableRef CVarIndicesToCalculatePerBucket(
	TEXT("Sequencer.MotionTrailTickIterations"),
	UE::SequencerAnimTools::FFrameCalculator::IndicesToCalculatePerBucket,
	TEXT("Number of iterations to tick sequencer when calculating motion trails"));

namespace UE
{
namespace SequencerAnimTools
{

/**********************************************************************************
*
*    FTrailspaceTransform
*
**********************************************************************************/

TOptional<FVector2D> FTrailScreenSpaceTransform::ProjectPoint(const FVector& Point) const
{

	FVector2D PixelLocation;
	const FVector4 ScreenPoint = View->WorldToScreen(Point);
	if (ScreenPoint.W > 0.0)
	{
		if (View->ScreenToPixel(ScreenPoint, PixelLocation))
		{
			PixelLocation /= DPIScale;
			return PixelLocation;
		}
	}
	return TOptional<FVector2D>();
}

/**********************************************************************************
*
*    FFrameCalculator
*
**********************************************************************************/

void FFrameCalculator::SetUpFrameCalculator(const UE::AIE::FFrameTimeByIndex& InCurrentFrameTimes, TRange<FFrameNumber> ViewRange)
{
	CurrentFrameTimes = InCurrentFrameTimes;
	//first makes sure viewrange is within eval range.
	if (ViewRange.GetLowerBoundValue() < InCurrentFrameTimes.StartFrame)
	{
		ViewRange.SetLowerBoundValue(InCurrentFrameTimes.StartFrame);
	}
	if (ViewRange.GetUpperBoundValue() > InCurrentFrameTimes.EndFrame)
	{
		ViewRange.SetUpperBoundValue(InCurrentFrameTimes.EndFrame);
	}

	TRange<FFrameNumber> Range = ViewRange;
	Ranges.SetNum(0);
	Ranges.Add(Range);
	CurrentRangeStartIndex = 0;
	CurrentRangeEndIndex = InCurrentFrameTimes.NumFrames - 1;

	Reset();
	CalculateStartEndIndices();
	NumOfBuckets = FMath::FloorToInt32((float)(CurrentRangeEndIndex + 1 - CurrentRangeStartIndex) / (float)(IndicesToCalculatePerBucket)) + 1;
}

void FFrameCalculator::CalculateStartEndIndices()
{
	if (CurrentRange < Ranges.Num())
	{
		TRange<FFrameNumber>& Range = Ranges[CurrentRange];
		CurrentRangeStartIndex = CurrentFrameTimes.CalculateIndex(Range.GetLowerBoundValue());
		CurrentRangeEndIndex = CurrentFrameTimes.CalculateIndex(Range.GetUpperBoundValue());
	}

}
void FFrameCalculator::AddMustHaveIndices(const TArray<int32>& InMustHaveIndices)
{
	MustHaveIndices = InMustHaveIndices;
}

void FFrameCalculator::Reset()
{
	CurrentRange = 0;
	CurrentBucket = 0; //-1 bucket is just MustHaveTimes, only for mouse down
	IndicesToCalculate.SetNum(0);
}

//calculate next set of indices in the IndicesToCalculate array
//will return true if we need to calculate, will return false if this is the last time
bool FFrameCalculator::CalculateIndices()
{
	IndicesToCalculate.SetNum(0);
	//any mouse press just do base bucket
	if (FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton) == true		||
		FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::MiddleMouseButton) == true	||
		FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::RightMouseButton) == true
		)
	{
		CurrentBucket = -1;
		CurrentRange = 0;
		return true;
	}
	if (CurrentBucket == -1 && MustHaveIndices.Num() > 0)
	{
		IndicesToCalculate.Append(MustHaveIndices);
		++CurrentBucket;
	}

	for (int32 Index = 0; Index < IndicesToCalculatePerBucket; ++Index)
	{
		const int32 CalculatedIndex = CurrentRangeStartIndex + CurrentBucket + (Index * NumOfBuckets);
		if (CalculatedIndex <= CurrentRangeEndIndex) //can go over
		{
			IndicesToCalculate.Add(CalculatedIndex);
		}
	}
	++CurrentBucket;
	if (CurrentBucket > NumOfBuckets)
	{
		++CurrentRange;
		if (CurrentRange >= Ranges.Num())
		{
			return false;
		}
		CalculateStartEndIndices();
	}
	return true;
}
/**********************************************************************************
*
*    FCurrentFramesInfo
*
**********************************************************************************/

void FCurrentFramesInfo::SetViewRange(const TRange<FFrameNumber>& InViewRange, bool bInViewRangeIsEvalRange)
{
	ViewRange = InViewRange;
	bViewRangeIsEvalRange = bInViewRangeIsEvalRange;
}
void FCurrentFramesInfo::SetUpFrameTimes(const TRange<FFrameNumber>& InEvalFrameRange, const FFrameNumber& InFrameStep)
{
	CurrentFrameTimes = UE::AIE::FFrameTimeByIndex(InEvalFrameRange.GetLowerBoundValue(), InEvalFrameRange.GetUpperBoundValue(), InFrameStep);
	SortedTransformIndices.Reserve(CurrentFrameTimes.NumFrames);
	TransformIndices.Reserve(CurrentFrameTimes.NumFrames);
	TransformIndices.Reset();

	FrameCalculator.SetUpFrameCalculator(CurrentFrameTimes, InEvalFrameRange);
}

void FCurrentFramesInfo::AddMustHaveTimes(const TSet<FFrameNumber>& InMustHaveTimes, const FFrameNumber& CurrentFrame)
{
	TArray<int32> MustHaveIndices;
	int32 CurrentIndex = CurrentFrameTimes.CalculateIndex(CurrentFrame);
	for (const FFrameNumber& FrameNumber : InMustHaveTimes)
	{
		int32 Index = CurrentFrameTimes.CalculateIndex(FrameNumber);
		if (Index != CurrentIndex)
		{
			MustHaveIndices.Add(Index);
		}
	}
	//current frame last so it doesn't flash on certain rigs
	MustHaveIndices.Add(CurrentIndex);
	FrameCalculator.AddMustHaveIndices(MustHaveIndices);
}

const TArray<int32>& FCurrentFramesInfo::IndicesToCalculate() const
{
	return FrameCalculator.IndicesToCalculate;
}

bool FCurrentFramesInfo::KeepCalculating()
{
	const bool bKeep = FrameCalculator.CalculateIndices();
	for (int32 Index : FrameCalculator.IndicesToCalculate)
	{
		const FFrameNumber FrameNumber = CurrentFrameTimes.CalculateFrame(Index);
		SortedTransformIndices.Add(Index, FrameNumber);
	}
	SortedTransformIndices.GenerateKeyArray(TransformIndices);
	SortedTransformIndices.GenerateValueArray(CurrentFrames);
	return bKeep;
}

void FCurrentFramesInfo::Reset()
{
	FrameCalculator.Reset();
	SortedTransformIndices.Reset();
	TransformIndices.Reset();
	CurrentFrames.SetNum(0);
}

/**********************************************************************************
*
*    FTrajectoryDrawInfo
*
**********************************************************************************/
void FTrajectoryDrawInfo::GetTrajectoryPointsForDisplay(const FTransform& OffsetTransform, const FTransform& ParentSpaceTransform, const FCurrentFramesInfo& InCurrentFramesInfo, bool bIsEvaluating,  TArray<FVector>& OutPoints, TArray<FFrameNumber>&  OutFrames)
{
	if (ArrayOfTransforms.IsValid() == false || ArrayOfTransforms->Transforms.Num() == 0 || InCurrentFramesInfo.TransformIndices.Num() == 0 || InCurrentFramesInfo.TransformIndices.Num() != InCurrentFramesInfo.CurrentFrames.Num())
	{
		return;
	}
	//if not evaluating we just sample the times that are in range
	//if we are evaluating then we use the InCurrentFramesInfo.TransformIndices to get the sparse values
	if (bIsEvaluating == false )
	{
		OutPoints.Reserve(InCurrentFramesInfo.TransformIndices.Num());
		OutPoints.SetNum(0);
		OutFrames.Reserve(InCurrentFramesInfo.TransformIndices.Num());
		OutFrames.SetNum(0);
		const int32 StartIndex = InCurrentFramesInfo.CurrentFrameTimes.CalculateIndex(InCurrentFramesInfo.ViewRange.GetLowerBoundValue());
		const int32 EndIndex = InCurrentFramesInfo.CurrentFrameTimes.CalculateIndex(InCurrentFramesInfo.ViewRange.GetUpperBoundValue());
		for (int64 Index = StartIndex; Index <= EndIndex; ++Index)
		{
			const FFrameNumber Frame = InCurrentFramesInfo.CurrentFrameTimes.CalculateFrame(Index);
			FVector Position = ParentSpaceTransform.TransformPosition(ArrayOfTransforms->Transforms[Index].GetLocation());
			Position = OffsetTransform.TransformPosition(Position);
			OutPoints.Add(Position);
			OutFrames.Add(Frame);
		}
	}
	else //if (bIsEvaluating)
	{
		if (InCurrentFramesInfo.bViewRangeIsEvalRange)
		{
			OutPoints.SetNum(InCurrentFramesInfo.TransformIndices.Num());
			OutFrames.SetNum(InCurrentFramesInfo.TransformIndices.Num());
			for (int32 Index = 0; Index < InCurrentFramesInfo.TransformIndices.Num(); ++Index)
			{
				const int32 TransformIndex = InCurrentFramesInfo.TransformIndices[Index];
				FVector Position = ParentSpaceTransform.TransformPosition(ArrayOfTransforms->Transforms[TransformIndex].GetLocation());
				Position = OffsetTransform.TransformPosition(Position);
				OutPoints[Index] = Position;
				OutFrames[Index] = InCurrentFramesInfo.CurrentFrames[Index];
			}
		}
		else
		{
			OutPoints.Reserve(InCurrentFramesInfo.TransformIndices.Num());
			OutPoints.SetNum(0);
			OutFrames.Reserve(InCurrentFramesInfo.TransformIndices.Num());
			OutFrames.SetNum(0);
			const int32 StartIndex = InCurrentFramesInfo.CurrentFrameTimes.CalculateIndex(InCurrentFramesInfo.ViewRange.GetLowerBoundValue());
			const int32 EndIndex = InCurrentFramesInfo.CurrentFrameTimes.CalculateIndex(InCurrentFramesInfo.ViewRange.GetUpperBoundValue());
			for (int32 Index = 0; Index < InCurrentFramesInfo.TransformIndices.Num(); ++Index)
			{
				const int32 TransformIndex = InCurrentFramesInfo.TransformIndices[Index];
				if (TransformIndex >= StartIndex && TransformIndex <= EndIndex)
				{
					FVector Position = ParentSpaceTransform.TransformPosition(ArrayOfTransforms->Transforms[TransformIndex].GetLocation());
					Position = OffsetTransform.TransformPosition(Position);
					OutPoints.Add(Position);
					OutFrames.Add(InCurrentFramesInfo.CurrentFrames[Index]);

				}
				else if (TransformIndex > EndIndex)
				{
					break;
				}
			}
		}
	}
}

void FTrajectoryDrawInfo::GetTickPointsForDisplay(const FTransform& OffsetTransform, const FTransform& ParentSpaceTransform, const FTrailScreenSpaceTransform& InScreenSpaceTransform, const FCurrentFramesInfo& InCurrentFramesInfo, bool bIsEvaluating, TArray<FVector2D>& Ticks, TArray<FVector2D>& TickNormals)
{
	if (ArrayOfTransforms.IsValid() == false || ArrayOfTransforms->Transforms.Num() == 0 || InCurrentFramesInfo.TransformIndices.Num() == 0 ||  InCurrentFramesInfo.TransformIndices.Num() != InCurrentFramesInfo.CurrentFrames.Num())
	{
		return;
	}
	const FVector PrevInterpolated = GetPoint(OffsetTransform, ParentSpaceTransform, InCurrentFramesInfo, InCurrentFramesInfo.CurrentFrameTimes.StartFrame);
	TOptional<FVector2D> PrevProjected = InScreenSpaceTransform.ProjectPoint(PrevInterpolated);
	const int32 StartIndex = InCurrentFramesInfo.CurrentFrameTimes.CalculateIndex(InCurrentFramesInfo.ViewRange.GetLowerBoundValue());
	const int32 EndIndex = InCurrentFramesInfo.CurrentFrameTimes.CalculateIndex(InCurrentFramesInfo.ViewRange.GetUpperBoundValue());
	for (int64 Index = StartIndex; Index < EndIndex; ++Index)
	{
		const FFrameNumber Frame = InCurrentFramesInfo.CurrentFrameTimes.CalculateFrame(Index);
		FVector Interpolated = GetPoint(OffsetTransform, ParentSpaceTransform, InCurrentFramesInfo, Frame);

		TOptional<FVector2D> Projected = InScreenSpaceTransform.ProjectPoint(Interpolated);

		if (Projected && PrevProjected)
		{
			Ticks.Add(Projected.GetValue());

			FVector2D Diff = Projected.GetValue() - PrevProjected.GetValue();
			Diff.Normalize();

			TickNormals.Add(FVector2D(-Diff.Y, Diff.X));
		}
		PrevProjected = Projected;
	}
}

FVector FTrajectoryDrawInfo::GetPoint(const FTransform& OffsetTransform, const FTransform& ParentSpaceTransform, const FCurrentFramesInfo& InCurrentFramesInfo, const FFrameNumber& InTime)
{
	if (InCurrentFramesInfo.TransformIndices.Num() > 0 && (InCurrentFramesInfo.TransformIndices.Num() == InCurrentFramesInfo.CurrentFrames.Num()))
	{
		FTransform Transform = ArrayOfTransforms->Interp(InTime, InCurrentFramesInfo.TransformIndices, InCurrentFramesInfo.CurrentFrames);
		FVector Position = ParentSpaceTransform.TransformPosition(Transform.GetLocation());
		Position = OffsetTransform.TransformPosition(Position);

		return Position;
	}
	return FVector(0.0,0.0,0.0);
}

} // namespace SequencerAnimTools
} // namespace UE

