// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputCoreTypes.h"
#include "UObject/Interface.h"
#include "BaseSequencerAnimTool.generated.h"

#define UE_API SEQUENCERANIMTOOLS_API

class UInteractiveToolManager;
class UTransformProxy;
class UCombinedTransformGizmo;
class UTransformGizmo;
class UInteractiveGizmoManager;

UINTERFACE(MinimalAPI)
class UBaseSequencerAnimTool : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IBaseSequencerAnimTool
{
	GENERATED_IINTERFACE_BODY()
	virtual bool ProcessCommandBindings(const FKey Key, const bool bRepeat) const { return false; }

};

struct FSequencerAnimToolHelpers
{
	struct FGizmoData
	{
		void* Owner = nullptr;
		UInteractiveToolManager* ToolManager = nullptr;
		UTransformProxy* TransformProxy = nullptr;
		UInteractiveGizmoManager* GizmoManager= nullptr;
		FString InstanceIdentifier;
	};
	static UE_API void CreateGizmo(const FGizmoData& InData, TObjectPtr <UCombinedTransformGizmo>& OutCombinedGizmo, TObjectPtr<UTransformGizmo>& OutTRSGizmo);
};

#undef UE_API
