// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Engine/World.h"
#include "InteractiveToolBuilder.h"
#include "MultiSelectionTool.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Framework/Commands/Commands.h"
#include "ISequencer.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"
#include "BaseSequencerAnimTool.h"
#include "TrailHierarchy.h"
#include "MotionTrailTool.generated.h"

class UInteractiveGizmoManager;
class UTransformProxy;
enum class EToolShutdownType : uint8;
struct FKey;

class USingleClickInputBehavior;
class UClickDragInputBehavior;
class UCombinedTransformGizmo;
class FEditorModeTools;
class USingleKeyCaptureBehavior;
class UTransformGizmo;

// Delegate for plugins to register additional trail hierarchies with UMotionTrailTool.
// Fired during SetupIntegration() after the standard FSequencerTrailHierarchy instances are created.
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCreateAdditionalTrailHierarchies,
	TArray<TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy>>&, TWeakPtr<ISequencer>);


/** 
* IClickBehaviorTarget Wrapper to handle left and right clicks
*/
class FLeftRghtClickBehaviorTarget: public IClickBehaviorTarget
{


public:
	FLeftRghtClickBehaviorTarget() {};

	IClickBehaviorTarget* Target = nullptr;
	bool bIsLeft = true;
	//Target will check these and then revert them back after using them.
	bool bClicked = false;

	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override
	{
		bClicked = true;
		FInputRayHit Hit = Target->IsHitByClick(ClickPos);
		bClicked = false;
		return Hit;
	}

	virtual void OnClicked(const FInputDeviceRay& ClickPos) override
	{
		bClicked = true;
		Target->OnClicked(ClickPos);
		bClicked = false;
	}

	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override
	{
		bClicked = true;
		Target->OnUpdateModifierState(ModifierID, bIsOn);
		bClicked = false;
	}

};


/**
 * Builder for UMotionTrailTool
 */
UCLASS()
class UMotionTrailToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override { return true; }
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

};

class FMotionTrailCommands : public TCommands<FMotionTrailCommands>
{
public:
	FMotionTrailCommands()
		: TCommands<FMotionTrailCommands>(
			"MotionTrailCommands",
			NSLOCTEXT("MotionTrailTool", "MotionTrailCommands", "Motion Trail Commands"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		)
	{}


	virtual void RegisterCommands() override;
	static const TMap<FName, TArray<TSharedPtr<FUICommandInfo>>>& GetCommands()
	{
		return FMotionTrailCommands::Get().Commands;
	}

public:
	/** Move selected keys one frame left*/
	TSharedPtr<FUICommandInfo> TranslateSelectedKeysLeft;

	/** Move seleced keys one frame right*/
	TSharedPtr<FUICommandInfo> TranslateSelectedKeysRight;

	/** Frame the selection*/
	TSharedPtr<FUICommandInfo> FrameSelection;

	/** Deselect All*/
	TSharedPtr<FUICommandInfo> DeselectAll;

	TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> Commands;
};


UCLASS()
class UMotionTrailTool : public UMultiSelectionTool, public IClickBehaviorTarget, public IBaseSequencerAnimTool
{
	GENERATED_BODY()

public:

	// IClickBehaviorTarget interface
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos);
	virtual void OnClicked(const FInputDeviceRay& ClickPos);

	// IModifierToggleBehaviorTarget implementation, inherited through IClickBehaviorTarget
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

	// UInteractiveTool overrides
	virtual	void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	virtual void OnTick(float DeltaTime) override;

	virtual TArray<UObject*> GetToolProperties(bool bEnabledOnly = true) const override;
	
	//IBaseSequencerAnimTool
	virtual bool ProcessCommandBindings(const FKey Key, const bool bRepeat) const override;

	// End interfaces

	void SetWorldGizmoModeManager(UWorld* World, UInteractiveGizmoManager* InGizmoManager, FEditorModeTools* InModeManager) 
	{
		TargetWorld = World;
		GizmoManager = InGizmoManager; 
		ModeManager = InModeManager;
	}
	UWorld* GetWorld() const { return TargetWorld; }

	FEditorModeTools* GetModeManager() const { return ModeManager;}
	static FString TrailKeyTransformGizmoInstanceIdentifier;
public:

	// Support for gizmo. Since the points aren't individual components, we don't actually use UTransformProxy
	// for the transform forwarding- we just use it for the callbacks.
	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo;
	UPROPERTY()
	TObjectPtr<UTransformGizmo> TRSGizmo = nullptr;

	FTransform StartDragTransform;
	bool bGizmoBeingDragged = false;

	FTransform GizmoTransform = FTransform::Identity;
	
	UE::SequencerAnimTools::FTrailHierarchy* GetHierarchyForSequencer(ISequencer* Sequencer) const { return SequencerHierarchies[Sequencer]; }
	const TArray<TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy>>& GetHierarchies() const { return TrailHierarchies; }

	SEQUENCERANIMTOOLS_API void AddTrailHierarchy(TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy> InHierarchy);

	// Plugins register callbacks here to add custom trail hierarchies during SetupIntegration().
	SEQUENCERANIMTOOLS_API static FOnCreateAdditionalTrailHierarchies OnCreateAdditionalTrailHierarchies;

	void SelectNone();
protected:

	void SetupIntegration();
	void ShutdownIntegration();
	void UpdateGizmoVisibility();
	void UpdateGizmoLocation();

	void GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);
	void GizmoTransformStarted(UTransformProxy* Proxy);
	void GizmoTransformEnded(UTransformProxy* Proxy);

protected:
	//Actions
	void TranslateSelectedKeysLeft();
	void TranslateSelectedKeysRight();
	void FrameSelection();
	void DeleteSelectedKeys();
	bool SomeKeysAreSelected() const;

protected:
	TArray<TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy>> TrailHierarchies;
	TMap<ISequencer*, UE::SequencerAnimTools::FTrailHierarchy*> SequencerHierarchies;

	FDelegateHandle OnSequencersChangedHandle;
	//ed mode for manips
	bool bManipulatorMadeChange = false;
	int32 TransactionIndex = -1;

protected:

	FLeftRghtClickBehaviorTarget LeftTarget;
	FLeftRghtClickBehaviorTarget RightTarget;

	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> LeftClickBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> RightClickBehavior = nullptr;



	UWorld* TargetWorld = nullptr;
	UInteractiveGizmoManager* GizmoManager = nullptr;
	FEditorModeTools* ModeManager = nullptr;
	// Support for Alt,Shift and Ctrl toggles
	int32 ShiftModifierId = 1;
	bool bShiftModifierOn = false;
	int32 CtrlModifierId = 2;
	bool bCtrlModifierOn = false;
	int32 AltModiferId = 3;
	bool bAltModifierOn = false;

private:
	bool IsHierarchyActive(const UE::SequencerAnimTools::FTrailHierarchy* Hierarchy) const;

	// Iterate all active hierarchies. The callable receives FTrailHierarchy&.
	template<typename Func>
	void ForEachActiveHierarchy(Func&& InFunc) const
	{
		for (const TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy>& Hierarchy : TrailHierarchies)
		{
			if (IsHierarchyActive(Hierarchy.Get()))
			{
				InFunc(*Hierarchy);
			}
		}
	}

	// Iterate active hierarchies with early-out. The callable returns bool:
	// true to stop iteration, false to continue. Returns true if stopped early.
	template<typename Func>
	bool ForEachActiveHierarchyUntil(Func&& InFunc) const
	{
		for (const TSharedPtr<UE::SequencerAnimTools::FTrailHierarchy>& Hierarchy : TrailHierarchies)
		{
			if (IsHierarchyActive(Hierarchy.Get()))
			{
				if (InFunc(*Hierarchy))
				{
					return true;
				}
			}
		}
		return false;
	}

	TSharedPtr<FUICommandList> CommandBindings;

};
