// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "InteractiveToolBuilder.h"
#include "MultiSelectionTool.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "Framework/Commands/Commands.h"
#include "BaseSequencerAnimTool.h"
#include "EditMode/ControlRigEditModeUtil.h"
#include "ScopedTransaction.h"
#include "Sequencer/EditModeAnimationUtil.h"
#include "Styling/AppStyle.h"
#include "Transform/TransformConstraintUtil.h"

#include "SequencerAnimEditPivotTool.generated.h"

class FControlRigInteractionScope;
class SWidget;
class UControlRig;
class UInteractiveGizmoManager;
class UTransformProxy;
enum class EToolShutdownType : uint8;
struct FKey;

class USingleClickInputBehavior;
class UClickDragInputBehavior;
class UCombinedTransformGizmo;
class ULevelSequence;
struct FRigControlElement;
class ISequencer;
class FUICommandList;
class IAssetViewport;
class UTransformGizmo;

/*
*  The way this sequencer pivot tool works is that
*  it will use modify the incoming selections temp pivot
*  while the mode is active. Reselecting will turn off the mode.
*
*/

/**
 * Builder for USequencerPivotTool
 */
UCLASS()
class  USequencerPivotToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};

struct FSelectionDuringDrag
{
	ULevelSequence* LevelSequence;
	FTransform CurrentTransform;
};

struct FControlRigSelectionDuringDrag : public FSelectionDuringDrag
{
	UControlRig* ControlRig;
	FName ControlName;
};

struct FActorSelectonDuringDrag : public FSelectionDuringDrag
{
	AActor* Actor;
};

struct FControlRigMappings
{
	TWeakObjectPtr<UControlRig> ControlRig;

	TOptional<FTransform> GetWorldTransform(const FName& Name) const;
	void SetFromWorldTransform(const FName& Name, const FTransform& WorldTransform);
	TArray<FName> GetAllControls() const;
	void SelectControls();
	bool IsAnyControlDeselected() const;
private:
	TMap<FName, FTransform> PivotTransforms;
};

struct FActorMappings
{
	TWeakObjectPtr<AActor> Actor;

	TOptional<FTransform> GetWorldTransform() const;
	void SetFromWorldTransform(const FTransform& WorldTransform);
	void SelectActors();
private:
	FTransform PivotTransform;
};

struct FSavedMappings
{
	TMap<TWeakObjectPtr<UControlRig>, FControlRigMappings> ControlRigMappings;
	TMap<TWeakObjectPtr<AActor>, FActorMappings> ActorMappings;
};

struct FLastSelectedObjects
{
	TArray<FControlRigMappings> LastSelectedControlRigs;
	TArray<FActorMappings> LastSelectedActors;

};
class FEditPivotCommands : public TCommands<FEditPivotCommands>
{
public:
	FEditPivotCommands()
		: TCommands<FEditPivotCommands>(
			"SequencerEditPivotTool",
			NSLOCTEXT("SequencerEditPivotTool", "SequencerEditPivotTool", "Edit Pivot Commands"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
			)
	{}


	virtual void RegisterCommands() override;
	static const TMap<FName, TArray<TSharedPtr<FUICommandInfo>>>& GetCommands()
	{
		return FEditPivotCommands::Get().Commands;
	}

public:
	/** Reset the Pivot*/
	TSharedPtr<FUICommandInfo> ResetPivot;
	/** Toggle Free/Pivot Mode*/
	TSharedPtr<FUICommandInfo> ToggleFreePivot;

	TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> Commands;
};


/**
Pivot tool class
 */
UCLASS()
class  USequencerPivotTool : public UMultiSelectionTool, public IClickBehaviorTarget  , public IBaseSequencerAnimTool
{
	GENERATED_BODY()

public:

	// UInteractiveTool overrides
	virtual void SetWorld(UWorld* World, UInteractiveGizmoManager* GizmoManager);
	virtual void Setup() override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }
	virtual void OnTick(float DeltaTime) override;

	// IClickBehaviorTarget interface
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	//IBaseSequencerAnimTool
	virtual bool ProcessCommandBindings(const FKey Key, const bool bRepeat) const override;

	// End interfaces

	//If In Pivot Mode, if not then in Free Mode
	bool IsInPivotMode() const { return bInPivotMode; }
	//Toggle Pivot Mode
	void TogglePivotMode() { SetPivotMode(!bInPivotMode); }
	//Set Pivot Mode
	void SetPivotMode(bool bVal);
protected:

	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> ClickBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;
	
	UPROPERTY()
	TObjectPtr<UTransformGizmo> TRSGizmo = nullptr;

protected:
	bool bInPivotMode = false; //pivot may be in 'free' mode or 'pivot' mode
	bool bShiftPressedWhenStarted = false;
	bool bCtrlPressedWhenStarted = false;
	UWorld* TargetWorld = nullptr;		// target World we will raycast into
	UInteractiveGizmoManager* GizmoManager = nullptr;

	FTransform StartDragTransform;
	bool bGizmoBeingDragged = false;
	bool bManipulatorMadeChange = false;
	TUniquePtr<FScopedTransaction> Transaction = nullptr;
	TArray<FControlRigSelectionDuringDrag> ControlRigDrags;
	TArray<FActorSelectonDuringDrag> ActorDrags;
	TArray<TSharedPtr<FControlRigInteractionScope>> InteractionScopes;

	//since we are selection based we can cache this
	ULevelSequence* LevelSequence;
	TArray<TWeakObjectPtr<UControlRig>> ControlRigs;
	TWeakPtr<ISequencer> SequencerPtr;
	TArray<TWeakObjectPtr<AActor>> Actors;

	FTransform GizmoTransform = FTransform::Identity;
	bool bPickingPivotLocation = false; //mauy remove
	void UpdateGizmoVisibility();
	void UpdateGizmoTransform();
	void UpdateTransformAndSelectionOnEntering();
	bool SetGizmoBasedOnSelection(bool bUseSaved = true);

	FInputRayHit FindRayHit(const FRay& WorldRay, FVector& HitPos);		// raycasts into World

	// Callbacks we'll receive from the gizmo proxy
	void GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);
	void GizmoTransformStarted(UTransformProxy* Proxy);
	void GizmoTransformEnded(UTransformProxy* Proxy);

	//Handle Selection and Pivot Location
	void SavePivotTransforms();
	void SaveLastSelected();

	// selection delegates
	void DeactivateMe();
	void RemoveDelegates();
	void HandleControlSelected(UControlRig* Subject, FRigControlElement* InControl, bool bSelected);
	void OnEditorSelectionChanged(UObject* NewSelection);
	FDelegateHandle OnEditorSelectionChangedHandle;

	// Functions and variables for handling the overlay for switching between free and pivot mode
	void CreateAndShowPivotOverlay();
	void RemoveAndDestroyPivotOverlay();

	FMargin GetPivotOverlayPadding() const;
	
	static FVector2f LastPivotOverlayLocation;
	TSharedPtr<SWidget> PivotWidget;
	TOptional<bool> bSetShowWidget;
public:
	void TryRemovePivotOverlay();
	void TryShowPivotOverlay();
	void UpdatePivotOverlayLocation(const FVector2D InLocation, TSharedPtr<IAssetViewport> ActiveLevelViewport);

private:
	TSharedPtr<FUICommandList> CommandBindings;
	void ResetPivot();

	// Stores data related to InControlRig to prepare its manipulation
	void PrepareRigForManipulation(UControlRig* InControlRig);
	
	// Stores data related to InActor to prepare its manipulation
	void PrepareActorForManipulation(AActor* InActor);
	
	// Updates a control using provided FControlRigSelectionDuringDrag data.
	void SetControlTransform(const FControlRigSelectionDuringDrag& InControlDrag);
	
	// Current transform context.
	FControlRigInteractionTransformContext TransformContext;
	
	// Used to store and apply keyframes. (if deferred)
	UE::AnimationEditMode::FControlRigKeyframer Keyframer;
	
	// Per control constraints cache.
	UE::TransformConstraintUtil::FConstraintsInteractionCache ConstraintsCache;
	
	// Per ControlRig dependencies between the selected controls during interaction.
	TMap<UControlRig*, UE::ControlRigEditMode::FInteractionDependencyCache> InteractionDependencies;
	
	// Updates the components bound to the control rigs for a responsive feedback
	void UpdateBoundComponents();

public:
	static FSavedMappings SavedPivotLocations;
	static FLastSelectedObjects LastSelectedObjects;
};
