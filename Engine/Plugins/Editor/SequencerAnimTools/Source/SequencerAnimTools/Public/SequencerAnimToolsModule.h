// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FUICommandList;

namespace UE
{
namespace SequencerAnimTools
{


class FSequencerAnimToolsModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TSharedPtr<FUICommandList> CommandBindings;

	//delegates
	void OnMotionTralOptionChanged(FName PropertyName);
};

} // namespace SequencerAnimTools
} // namespace UE
