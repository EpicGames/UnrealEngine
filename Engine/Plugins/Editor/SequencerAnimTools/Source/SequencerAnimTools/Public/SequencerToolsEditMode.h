// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/LegacyEdModeWidgetHelpers.h"

#include "SequencerToolsEditMode.generated.h"


UCLASS()
class USequencerToolsEditMode : public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()
public:
	SEQUENCERANIMTOOLS_API static FEditorModeID ModeName;

	USequencerToolsEditMode();
	virtual ~USequencerToolsEditMode();

	// UEdMode interface
	virtual void Enter();
	virtual void Exit();

	// UBaseLegacyWidgetEdMode interface
	
	virtual bool UsesToolkits() const  override { return false; }
	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;
	virtual bool UsesTransformWidget() const override { return false; }
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
private:

	void LocalOnModeActivated(const FEditorModeID& InID, bool bIsActive); //UEdMode has this already defined but it's not virtual

	bool bDeactivateOnPIEStartStateToRestore;
	bool bDeactivateOnSaveWorldToRestore;
};
