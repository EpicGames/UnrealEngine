// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Math/Box.h"
#include "TrajectoryDrawInfo.h"
#include "Tools/MotionTrailOptions.h"
#include "HitProxies.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

class FCanvas;
namespace UE::SequencerAnimTools { class FTrailHierarchy; }
struct FConvexVolume;

class FEditorViewportClient;
class FPrimitiveDrawInterface;
class UMotionTrailEditorMode;
class UMotionTrailToolOptions;

namespace UE
{
namespace SequencerAnimTools
{
struct FInputClick
{
	FInputClick() {};
	FInputClick(bool bA, bool bC, bool bS) : bAltIsDown(bA), bCtrlIsDown(bC), bShiftIsDown(bS) {};
	bool bAltIsDown = false; 
	bool bCtrlIsDown = false;
	bool bShiftIsDown = false;
	bool bIsRightMouse = false;
};
struct HBaseTrailProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	FGuid Guid;
	HBaseTrailProxy(const FGuid& InGuid, EHitProxyPriority InPriority = HPP_Foreground) 
		:HHitProxy(InPriority)
		,Guid(InGuid)
	{}
};

struct HNewMotionTrailProxy : public HBaseTrailProxy
{
	DECLARE_HIT_PROXY();

	FVector Point;
	FFrameNumber CurrentFrame;

	HNewMotionTrailProxy(const FGuid& InGuid, const FVector& InPoint, const FFrameNumber& InFrame)
		:HBaseTrailProxy(InGuid)
		, Point(InPoint)
		, CurrentFrame(InFrame)
	{}

};

enum class ETrailCacheState : uint8
{
	UpToDate = 2,
	Stale = 1,
	Dead = 0,
	NotUpdated = 3
};

//Status of trail, including possibly frames that must get updated on next evaluation
struct FTrailCurrentStatus
{
	ETrailCacheState CacheState = ETrailCacheState::Stale;
	TArray<FFrameNumber> FramesMustUpdate;
};

//struct to cacluate color. The same struct is meant to be used in same loop, and will be set up by the owner(trail hierarchy).
struct FColorState
{
	bool bFirstFrame = true;
	FFrameNumber TicksPerFrame = FFrameNumber(100);
	FFrameNumber StartFrame;
	FFrameNumber SequencerTime;
	bool bIsPinned = false;
	EMotionTrailTrailStyle PinnedStyle;
	FLinearColor CalculatedColor = FColor(0xffffff);
	UMotionTrailToolOptions* Options = nullptr;

	SEQUENCERANIMTOOLS_API void Setup(FTrailHierarchy* TrailHierarchy);
	SEQUENCERANIMTOOLS_API void ReadyForTrail(bool bInIsPinned, EMotionTrailTrailStyle InPinnedStyle);
	SEQUENCERANIMTOOLS_API EMotionTrailTrailStyle GetStyle() const;
};

//main abstract class that trails implement
class FTrail
{
public:

	FTrail(UObject* InOwner)
		: Owner(InOwner)
		,CacheState(ETrailCacheState::Stale)
		, bForceEvaluateNextTick(true)
	{}

	virtual ~FTrail() = default;

	struct FNewSceneContext
	{
		bool bCheckForChange; 
		FGuid YourNode;
		class FTrailHierarchy* TrailHierarchy;
	};

	UObject* GetOwner() const
	{ 
		return (Owner.IsValid() ? Owner.Get() : nullptr);
	}
	//main function to update trail each tick
	virtual FTrailCurrentStatus UpdateTrail(const FNewSceneContext& NewSceneContext) = 0;
	//get the transform at a specified time
	virtual void Interp(const FFrameNumber& Time, FTransform& OutTransform, FTransform& OutParentTransform) { OutTransform = FTransform::Identity; OutParentTransform = FTransform::Identity;
	}
	//when multiple frame update is finally finished
	virtual void UpdateFinished(const TRange<FFrameNumber>& UpdatedRange, const TArray<int32>& IndicesToCalcluate, bool bDoneCalcuating) { ClearCachedData();  CacheState = ETrailCacheState::UpToDate; }
	virtual FText GetName() const = 0;

	//add set of important times for this trail to be used by the incremental evaluations, usually these are the edited times
	virtual void AddImportantTimes(TSet<FFrameNumber>& InOutImportantTimes) {};
	
	//Additional Render/HitTest event handling for specific trails, usually let default renderer handle it
	virtual void Render(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI, bool bTrailIsEvaluating) {};
	virtual void RenderEvaluating(const FGuid& Guid, const FSceneView* View, FPrimitiveDrawInterface* PDI) {};
	virtual void DrawHUD(const FSceneView* View, FCanvas* Canvas) {};
	virtual bool HandleClick(const FGuid& Guid, FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, FInputClick Click) { return false; }
	virtual bool IsAnythingSelected() const { return false; }
	virtual bool IsAnythingSelected(FVector& OutVectorPosition, FQuat& OutRotation)const { return false;}
	virtual bool IsAnythingSelected(TArray<FVector>& OutVectorPositions)const { return false; }
	virtual bool IsTrailSelected() const { return false; }
	virtual bool BoxSelect(FBox& InBox, bool InSelect = true) {return false;}
	virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true)  { return false; }
	virtual void SetOffsetMode() {};
	virtual bool GetOffsetMode() const {return false;}
	virtual bool StartTracking() { return false; }
	virtual bool ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset) { return false; }
	virtual bool EndTracking() { return false; }
	virtual bool IsTracking() const { return false; }
	virtual void TranslateSelectedKeys(bool bRight) {};
	virtual void DeleteSelectedKeys() {};
	virtual void SelectNone() {};
	virtual void UpdateKeysInRange(const TRange<FFrameNumber>& ViewRange) {};
	virtual void ClearCachedData() {};
	virtual void HasStartedEvaluating() {};
	virtual void ReadyToDrawTrail(FColorState& ColorState, const FCurrentFramesInfo* InCurrentFramesInfo,bool bIsEvaluating, bool bIsPinned) {};
	virtual void ActorChanged(AActor* InActor) {};
	SEQUENCERANIMTOOLS_API virtual bool HandleObjectsChanged(const TMap<UObject*, UObject*>& ReplacementMap);

	// Optionally implemented methods
	virtual TArray<FFrameNumber> GetKeyTimes() const { return TArray<FFrameNumber>(); }
	virtual TArray<FFrameNumber> GetSelectedKeyTimes() const { return TArray<FFrameNumber>(); }

	virtual void GetTrajectoryPointsForDisplay(const FCurrentFramesInfo& InCurrentFramesInfo, bool bIsEvaluating, TArray<FVector>& OutPoints, TArray<FFrameNumber>& OutFrames)
	{
		if (FTrajectoryDrawInfo* DI = GetDrawInfo())
		{
			DI->GetTrajectoryPointsForDisplay(FTransform::Identity, GetParentSpaceTransform(), InCurrentFramesInfo, bIsEvaluating, OutPoints, OutFrames);
		}
	}
	virtual void GetTickPointsForDisplay(const FTrailScreenSpaceTransform& InScreenSpaceTransform, const FCurrentFramesInfo& InCurrentFramesInfo, bool bIsEvaluating, TArray<FVector2D>& OutTicks, TArray<FVector2D>& OutTickTangents)
	{
		if (FTrajectoryDrawInfo* DI = GetDrawInfo())
		{
			DI->GetTickPointsForDisplay(FTransform::Identity, GetParentSpaceTransform(), InScreenSpaceTransform, InCurrentFramesInfo, bIsEvaluating, OutTicks, OutTickTangents);
		}
	}

	//offset the trail from current position
	virtual FTransform GetOffsetTransform() const { return FTransform::Identity; }
	virtual void ClearOffsetTransform() { ForceEvaluateNextTick(); };
	virtual bool HasOffsetTransform() { return !GetOffsetTransform().Equals(FTransform::Identity); }
	//set space and get space transform
	virtual FTransform GetParentSpaceTransform() const { return FTransform::Identity; }
	virtual void SetSpace(AActor* InActor, const FName& InComponentName) {};
	virtual void ClearSpace() {};

	virtual void ForceEvaluateNextTick() { bForceEvaluateNextTick = true; }

	SEQUENCERANIMTOOLS_API virtual void GetColor(const FFrameNumber& CurrentTime, FColorState& InOutColorState);
	//get and restore trail from it's data
	class FMotionTrailState
	{
	public:
		virtual ~FMotionTrailState() = default;
		virtual void RestoreTrail(FTrailHierarchy* TrailHierarchy) = 0;
	};
	virtual TUniquePtr<FMotionTrailState> GetMotionTrailState() const { return nullptr; }

	FTrajectoryDrawInfo* GetDrawInfo() const { return DrawInfo.Get(); }

	ETrailCacheState GetCacheState() const { return CacheState; }


protected:

	TWeakObjectPtr<UObject> Owner;
	ETrailCacheState CacheState;

	bool bForceEvaluateNextTick;
	TUniquePtr<FTrajectoryDrawInfo> DrawInfo;


};
} // namespace MovieScene
} // namespace UE
