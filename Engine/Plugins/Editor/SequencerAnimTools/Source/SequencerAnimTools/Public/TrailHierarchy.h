// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trail.h"
#include "Tools/TrailCategory.h"

class HHitProxy;
struct FFrameNumber;
class UMotionTrailToolOptions;
class FTrailHierarchy;

namespace UE
{
namespace SequencerAnimTools
{

struct FTrailVisibilityManager
{
	bool IsTrailVisible(const FGuid& Guid, const FTrail* Trail, bool bShowSelected = true) const
	{
		return !InactiveMask.Contains(Guid) && !VisibilityMask.Contains(Guid) && (AlwaysVisible.Contains(Guid) || (bShowSelected == true && (Selected.Contains(Guid) || ControlSelected.Contains(Guid)))
			|| Trail->IsAnythingSelected()) && Guid.IsValid();
	}

	bool IsTrailAlwaysVisible(const FGuid& Guid) const
	{
		return AlwaysVisible.Contains(Guid);
	}
	void SetTrailAlwaysVisible(const FGuid& Guid, bool bSet)
	{
		if (bSet)
		{
			AlwaysVisible.Add(Guid);
		}
		else
		{
			AlwaysVisible.Remove(Guid);
		}
	}

	void Reset()
	{
		InactiveMask.Empty();
		VisibilityMask.Empty();
		AlwaysVisible.Empty();
		Selected.Empty();
		ControlSelected.Empty();
	}

	TSet<FGuid> InactiveMask; // Any trails whose cache state or parent's cache state has been marked as NotUpdated
	TSet<FGuid> VisibilityMask; // Any trails masked out by the user interface, ex bone trails
	TSet<FGuid> AlwaysVisible; // Any trails pinned by the user interface
	TSet<FGuid> Selected; // Any transform or bone trails selected in the user interface
	TSet<FGuid> ControlSelected;// Any control rig trails selected
};

class ITrailHierarchyRenderer
{
public:
	virtual ~ITrailHierarchyRenderer() = default;
	virtual void Render(const FSceneView* View, FPrimitiveDrawInterface* PDI) = 0;
	virtual void DrawHUD(const FSceneView* View, FCanvas* Canvas) = 0;
};

class FTrailHierarchyRenderer : public ITrailHierarchyRenderer
{
public:
	FTrailHierarchyRenderer(FTrailHierarchy* InOwningHierarchy, UMotionTrailToolOptions* InOptions)
		: OwningHierarchy(InOwningHierarchy), CachedOptions(InOptions)
	{}

	SEQUENCERANIMTOOLS_API virtual void Render(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	SEQUENCERANIMTOOLS_API virtual void DrawHUD(const FSceneView* View, FCanvas* Canvas) override;

private:
	FTrailHierarchy* OwningHierarchy;
	UMotionTrailToolOptions* CachedOptions;

};

class FTrailHierarchy
{
public:

	FTrailHierarchy()
		:TickViewRange(TRange<FFrameNumber>(0,0))
		,TickEvalRange(TRange<FFrameNumber>(0,0))
		,TicksPerSegment(1)
		,LastTickEvalRange(TRange<FFrameNumber>(0,0))
		,LastTicksPerSegment(1)
		, AllTrails()
		, TimingStats()
		, VisibilityManager()
	{}

	virtual ~FTrailHierarchy() = default;

	FTrailHierarchy(const FTrailHierarchy&) = delete;
	FTrailHierarchy& operator=(const FTrailHierarchy&) = delete;

	virtual void Initialize() = 0;
	virtual void Destroy() = 0; // TODO: make dtor?
	virtual ITrailHierarchyRenderer* GetRenderer() const = 0;
	virtual FFrameNumber GetLocalTime() const = 0;
	virtual FFrameNumber GetFramesPerFrame() const = 0;
	virtual FFrameNumber GetFramesPerSegment() const = 0;

	virtual const FCurrentFramesInfo* GetCurrentFramesInfo() const = 0;
	SEQUENCERANIMTOOLS_API virtual bool IsVisible(const FGuid& InTrailGuid) const;
	virtual bool CheckForChanges() = 0;
	virtual bool IsTrailEvaluating(const FGuid& InTrailGuid, bool bIndirectlyOnly) const = 0;

	SEQUENCERANIMTOOLS_API virtual void CalculateEvalRangeArray();
	SEQUENCERANIMTOOLS_API virtual void Update();
	SEQUENCERANIMTOOLS_API virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, FInputClick Click);
	SEQUENCERANIMTOOLS_API virtual bool IsHitByClick(HHitProxy* HitProx);
	SEQUENCERANIMTOOLS_API virtual bool BoxSelect(FBox& InBox, bool InSelect = true);
	SEQUENCERANIMTOOLS_API virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true) ;

	SEQUENCERANIMTOOLS_API virtual bool IsAnythingSelected(FVector& OutVectorPosition, FQuat& OutRotation)const;
	SEQUENCERANIMTOOLS_API virtual bool IsAnythingSelected(TArray<FVector>& OutVectorPositions, bool bAllPositions = false) const;
	SEQUENCERANIMTOOLS_API virtual bool IsAnythingSelected() const;
	SEQUENCERANIMTOOLS_API virtual void SelectNone();
	SEQUENCERANIMTOOLS_API virtual bool IsSelected(const FGuid& Key) const;
	SEQUENCERANIMTOOLS_API virtual bool IsAlwaysVisible(const FGuid Key) const;

	SEQUENCERANIMTOOLS_API virtual void AddTrail(const FGuid& Key, TUniquePtr<FTrail>&& TrailPtr);
	SEQUENCERANIMTOOLS_API virtual void RemoveTrail(const FGuid& Key);

	SEQUENCERANIMTOOLS_API virtual bool StartTracking();
	SEQUENCERANIMTOOLS_API virtual bool ApplyDelta(const FVector& Pos, const FRotator& Rot, const FVector& WidgetLocation, bool bApplyToOffset);
	SEQUENCERANIMTOOLS_API virtual bool EndTracking();

	SEQUENCERANIMTOOLS_API virtual void TranslateSelectedKeys(bool bRight);
	SEQUENCERANIMTOOLS_API virtual void DeleteSelectedKeys();

	const TRange<FFrameNumber>& GetViewFrameRange() const { return TickViewRange; }
	FFrameNumber GetTicksPerSegment() const { return TicksPerSegment; }
	const TMap<FGuid, TUniquePtr<FTrail>>& GetAllTrails() const { return AllTrails; }

	const TMap<FString, FTimespan>& GetTimingStats() const { return TimingStats; };
	TMap<FString, FTimespan>& GetTimingStats() { return TimingStats; }

	FTrailVisibilityManager& GetVisibilityManager() { return VisibilityManager; }

	ETrailCategory GetTrailCategory() const { return TrailCategory; }

protected:
	SEQUENCERANIMTOOLS_API void RemoveTrailIfNotAlwaysVisible(const FGuid& Key);

	SEQUENCERANIMTOOLS_API void OpenContextMenu(const FGuid& TrailGuid);
protected:
	TRange<FFrameNumber> TickViewRange;
	TRange<FFrameNumber> TickEvalRange;

	FFrameNumber TicksPerSegment;
	TRange<FFrameNumber> LastTickEvalRange = TRange<FFrameNumber>(0, 0);
	FFrameNumber LastTicksPerSegment = FFrameNumber(1);

	TMap<FGuid, TUniquePtr<FTrail>> AllTrails;

	TMap<FString, FTimespan> TimingStats;

	FTrailVisibilityManager VisibilityManager;

	ETrailCategory TrailCategory = ETrailCategory::None;
};

} // namespace MovieScene
} // namespace UE
