// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "Misc/Guid.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Tools/EvaluateSequencerTools.h"
#include "Tools/MotionTrailOptions.h"

class FSceneView;
template <typename OptionalType> struct TOptional;

namespace UE
{
namespace SequencerAnimTools
{


class FTrailScreenSpaceTransform
{
public:
	FTrailScreenSpaceTransform(const FSceneView* InView,  const float InDPIScale = 1.0f)
		: View(InView)
		, DPIScale(InDPIScale)
	{}

	SEQUENCERANIMTOOLS_API TOptional<FVector2D> ProjectPoint(const FVector& Point) const;

private:
	const FSceneView* View;
	const float DPIScale;
};


//This struct will give set of indices to calculate based upon number of frames
//and current bucket we are calculating. t
struct FFrameCalculator
{
	FFrameCalculator()
	{
		IndicesToCalculate.Reserve(IndicesToCalculatePerBucket);
	}
	SEQUENCERANIMTOOLS_API void SetUpFrameCalculator(const UE::AIE::FFrameTimeByIndex& InCurrentFrameTimes, TRange<FFrameNumber> ViewRange);
	SEQUENCERANIMTOOLS_API void AddMustHaveIndices(const TArray<int32>& InMustHaveIndices);
	SEQUENCERANIMTOOLS_API void Reset();
	SEQUENCERANIMTOOLS_API bool CalculateIndices();

	SEQUENCERANIMTOOLS_API static int32 IndicesToCalculatePerBucket;
	UE::AIE::FFrameTimeByIndex CurrentFrameTimes;
	//set of ranges of time we calculate, if view range is same as eval range this will just be one range that's equal to the CurrentFrameTimesRange
	TArray<TRange<FFrameNumber>> Ranges;
	int32 CurrentRange;
	int32 CurrentRangeStartIndex;
	int32 CurrentRangeEndIndex;

	int32 NumOfBuckets;
	int32 CurrentBucket;
	TArray<int32> IndicesToCalculate;
	TArray<int32> MustHaveIndices;

private:
	SEQUENCERANIMTOOLS_API void CalculateStartEndIndices();

};

// Holds set of currently calculating frames
struct FCurrentFramesInfo
{
	SEQUENCERANIMTOOLS_API void SetViewRange(const TRange<FFrameNumber>& InViewRange, bool bInViewRangeIsEvalRange);
	SEQUENCERANIMTOOLS_API void SetUpFrameTimes(const TRange<FFrameNumber>& InEvalFrameRange, const FFrameNumber& InFrameStep);
	SEQUENCERANIMTOOLS_API void AddMustHaveTimes(const TSet<FFrameNumber>& InMustHaveTimes, const FFrameNumber& InFrameNumber);
	SEQUENCERANIMTOOLS_API const TArray<int32>& IndicesToCalculate() const;
	SEQUENCERANIMTOOLS_API bool KeepCalculating();
	SEQUENCERANIMTOOLS_API void Reset();

	//range and frame rate, this is fixed and will only change if evaluation range(sequencer range/display rate) changes.
	UE::AIE::FFrameTimeByIndex CurrentFrameTimes; 
	TRange<FFrameNumber> ViewRange;
	bool bViewRangeIsEvalRange = false;
	//this also contains the indices to calculate
	FFrameCalculator  FrameCalculator;

	//set of indices that slowly grow
	TSortedMap<int32, FFrameNumber> SortedTransformIndices;
	//cached array of the transform indices
	TArray<int32> TransformIndices;  
	// all trails share this now
	TArray<FFrameNumber> CurrentFrames; 
};


struct FTrajectoryDrawInfo
{
	FTrajectoryDrawInfo(EMotionTrailTrailStyle InStyle, const FLinearColor& InColor, TSharedPtr<UE::AIE::FArrayOfTransforms>& InArrayOfTransforms,
		TSharedPtr<UE::AIE::FArrayOfTransforms>& InParentSpaceArrayOfTransforms)
		: Color(InColor)
		, ArrayOfTransforms(InArrayOfTransforms)
		, ParentSpaceTransforms(InParentSpaceArrayOfTransforms)
	{}

	~FTrajectoryDrawInfo() = default;
	SEQUENCERANIMTOOLS_API void GetTrajectoryPointsForDisplay(const FTransform& OffsetTransform, const FTransform& ParentSpaceTransform, const FCurrentFramesInfo& InCurrentFramesInfo, bool bIsEvaluating,  TArray<FVector>& OutPoints, TArray<FFrameNumber>& OutFrames);
	SEQUENCERANIMTOOLS_API void GetTickPointsForDisplay(const FTransform& OffsetTransform, const FTransform& ParentSpaceTransform, const FTrailScreenSpaceTransform& ScreenSpaceTransform, const FCurrentFramesInfo & InCurrentFramesInfo, bool bIsEvaluating, TArray<FVector2D>& OutTicks, TArray<FVector2D>& OutTickTangents);
	SEQUENCERANIMTOOLS_API FVector GetPoint(const FTransform& OffsetTransform, const FTransform& ParentSpaceTransform, const FCurrentFramesInfo& InDisplayContext, const FFrameNumber& InTime);


	void SetColor(const FLinearColor& InColor) { Color = InColor; }
	FLinearColor GetColor() const { return Color; }
	void SetStyle(EMotionTrailTrailStyle InStyle) { Style = InStyle; }
	EMotionTrailTrailStyle GetStyle() const { return Style; }


protected:
	EMotionTrailTrailStyle Style;
	FLinearColor Color;
	TSharedPtr<UE::AIE::FArrayOfTransforms> ArrayOfTransforms;
	TSharedPtr<UE::AIE::FArrayOfTransforms> ParentSpaceTransforms;
};



} // namespace MovieScene
} // namespace UE
