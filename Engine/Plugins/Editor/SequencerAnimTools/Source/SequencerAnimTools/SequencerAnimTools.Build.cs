// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SequencerAnimTools: ModuleRules
{
	public SequencerAnimTools (ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "Slate",
                "SlateCore",
				"ControlRig",
				"ControlRigEditor"
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Constraints",
				"Engine",
				"InputCore",
				"UnrealEd",
				"LevelEditor",
				
				"EditorFramework",
                "EditorInteractiveToolsFramework",
                "InteractiveToolsFramework",

                "MovieScene",
                "MovieSceneTracks",
                "MovieSceneTools",
                "Sequencer",
				"LevelSequence",
				"LevelSequenceEditor",
			}
        );
	}
}
